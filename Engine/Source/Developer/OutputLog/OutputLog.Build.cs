// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class OutputLog : ModuleRules
{
	public OutputLog(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject", // @todo Mac: for some reason it's needed to link in debug on Mac
				"EngineSettings",
				"InputCore",
				"Slate",
				"SlateCore",
				"DesktopPlatform",
				"ToolWidgets",
				"ToolMenus",
			}
		);

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"EditorFramework",
					"StatusBar",
					"UnrealEd",
				}
			);
		}

		if (Target.bBuildEditor || Target.bBuildDeveloperTools)
		{
			PrivateIncludePathModuleNames.Add("WorkspaceMenuStructure");
			PrivateDependencyModuleNames.Add("TargetPlatform");
			PrivateDefinitions.Add("OUTPUTLOG_HAS_TARGET_PLATFORMS=1");
		}
		else
		{
			PrivateDefinitions.Add("OUTPUTLOG_HAS_TARGET_PLATFORMS=0");
		}

		if (Target.bCompileAgainstEngine)
		{
			// Required for output log drawer in editor / engine builds. 
			PrivateDependencyModuleNames.Add("Engine");
		}

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
