// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/Object.h"

#include "OutputLogMenuContext.generated.h"


class SConsoleInputBox;
class SOutputLog;


UCLASS()
class UConsoleInputBoxMenuContext : public UObject
{
	GENERATED_BODY()

public:
	void Init(const TSharedRef<SConsoleInputBox>& InConsoleInputBox)
	{
		WeakConsoleInputBox = InConsoleInputBox;
	}

	TSharedPtr<SConsoleInputBox> GetConsoleInputBox() const
	{
		return WeakConsoleInputBox.Pin();
	}

private:
	TWeakPtr<SConsoleInputBox> WeakConsoleInputBox;
};


UCLASS()
class UOutputLogMenuContext : public UObject
{
	GENERATED_BODY()

public:
	void Init(const TSharedRef<SOutputLog>& InOutputLog)
	{
		WeakOutputLog = InOutputLog;
	}

	TSharedPtr<SOutputLog> GetOutputLog() const
	{
		return WeakOutputLog.Pin();
	}

private:
	TWeakPtr<SOutputLog> WeakOutputLog;
};
