// Copyright Epic Games, Inc. All Rights Reserved.

#include "OutputLogModule.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Textures/SlateIcon.h"
#include "Framework/Docking/TabManager.h"
#include "Styling/AppStyle.h"
#include "OutputLogCreationParams.h"
#include "SDebugConsole.h"
#include "SOutputLog.h"
#include "SDeviceOutputLog.h"

#include "Widgets/Docking/SDockTab.h"
#include "Misc/ConfigCacheIni.h"

#if WITH_EDITOR
#include "Editor.h"
#include "ISettingsModule.h"
#endif

#if WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "OutputLogSettings.h"
#endif

#include "Internationalization/Internationalization.h"
#include "OutputLogStyle.h"

#if WITH_EDITOR
#include "StatusBarSubsystem.h"
#endif

IMPLEMENT_MODULE(FOutputLogModule, OutputLog);

namespace OutputLogModule
{
	static const FName OutputLogTabName = FName(TEXT("OutputLog"));
	static const FName DeviceOutputLogTabName = FName(TEXT("DeviceOutputLog"));

	bool bHideConsole = false;
	FAutoConsoleVariableRef CVarHideConsoleCommand(
		TEXT("OutputLogModule.HideConsole"), 
		bHideConsole, 
		TEXT("Whether debug console widgets should be hidden (false by default)"), 
		FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* /*CVar*/)
			{
				if (bHideConsole)
				{
					FOutputLogModule::Get().CloseDebugConsole();
				}
			}),
		ECVF_ReadOnly);
}

/** This class is to capture all log output even if the log window is closed */
class FOutputLogHistory : public FOutputDevice
{
public:

	FOutputLogHistory()
	{
		GLog->AddOutputDevice(this);
	}

	~FOutputLogHistory()
	{
		// At shutdown, GLog may already be null
		if (GLog != NULL)
		{
			GLog->RemoveOutputDevice(this);
		}
	}

	/** Gets all captured messages */
	const TArray< TSharedPtr<FOutputLogMessage> >& GetMessages() const
	{
		return Messages;
	}

	void Suspend()
	{
		bSuspended = true;
	}

	void Resume()
	{
		bSuspended = false;
	}
protected:

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
	{
		if (!bSuspended)
		{
			// Capture all incoming messages and store them in history
			SOutputLog::CreateLogMessages(V, Verbosity, Category, Messages);
		}
	}

private:

	/** All log messsges since this module has been started */
	TArray< TSharedPtr<FOutputLogMessage> > Messages;

	bool bSuspended = false;
};


TSharedRef<SDockTab> FOutputLogModule::SpawnOutputLogTab(const FSpawnTabArgs& Args)
{
	TSharedRef<SOutputLog> NewLog = SNew(SOutputLog, false)
			.Messages(&OutputLogHistory->GetMessages())
			.InitialLogFilterSettings(&GetDefault<UOutputLogSettings>()->OutputLogTabFilter);

	OutputLog = NewLog;

	TSharedRef<SDockTab> NewTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		.Label(NSLOCTEXT("OutputLog", "TabTitle", "Output Log"))
		.OnPersistVisualState_Raw(this, &FOutputLogModule::SaveDockedTabSettings)
		[
			MoveTemp(NewLog)
		];

	OutputLogTab = NewTab;

	return NewTab;
}

void FOutputLogModule::SaveDockedTabSettings()
{
	if (TSharedPtr<SOutputLog> SharedOutputLog = OutputLog.Pin())
	{
		// Cache the closing LogFilterTab so that we can restore the same filter when it's opened again
		UOutputLogSettings* Settings = GetMutableDefault<UOutputLogSettings>();
		SharedOutputLog->GetOutputLogFilter().ExportSettings(Settings->OutputLogTabFilter);
		Settings->SaveConfig();
	}
}

TSharedRef<SDockTab> FOutputLogModule::SpawnDeviceOutputLogTab(const FSpawnTabArgs& Args)
{
	return SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		.Label(NSLOCTEXT("OutputLog", "DeviceTabTitle", "Device Output Log"))
		[
			SNew(SDeviceOutputLog)
		];
}

void FOutputLogModule::StartupModule()
{
	FOutputLogStyle::Get();

#if WITH_EDITOR
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (SettingsModule)
	{
		SettingsModule->RegisterSettings("Editor", "General", "Output Log",
			NSLOCTEXT("OutputLog", "OutputLogSettingsName", "Output Log"),
			NSLOCTEXT("OutputLog", "OutputLogSettingsDescription", "Set up preferences for the Output Log appearance and workflow."),
			GetMutableDefault<UOutputLogSettings>()
		);
	}

	FEditorDelegates::BeginPIE.AddRaw(this, &FOutputLogModule::ClearOnPIE);
#endif

#if WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(OutputLogModule::OutputLogTabName, FOnSpawnTab::CreateRaw(this, &FOutputLogModule::SpawnOutputLogTab))
		.SetDisplayName(NSLOCTEXT("OutputLog", "OutputLogTab", "Output Log"))
		.SetTooltipText(NSLOCTEXT("OutputLog", "OutputLogTooltipText", "Open the Output Log tab."))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsLogCategory())
		.SetIcon(FSlateIcon(FOutputLogStyle::Get().GetStyleSetName(), "Log.TabIcon"));

	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(OutputLogModule::DeviceOutputLogTabName, FOnSpawnTab::CreateRaw(this, &FOutputLogModule::SpawnDeviceOutputLogTab))
		.SetDisplayName(NSLOCTEXT("OutputLog", "DeviceOutputLogTab", "Device Output Log"))
		.SetTooltipText(NSLOCTEXT("OutputLog", "DeviceOutputLogTooltipText", "Open the Device Output Log tab."))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsLogCategory())
		.SetIcon(FSlateIcon(FOutputLogStyle::Get().GetStyleSetName(), "Log.TabIcon"));
#endif

	OutputLogHistory = MakeShareable(new FOutputLogHistory);
	OutputLogFilterCache = MakeUnique<FOutputLogFilter>();
	ConsoleCommandExecutorPermissionList = MakeUnique<FNamePermissionList>();

	SOutputLog::RegisterSettingsMenu();
}

void FOutputLogModule::ShutdownModule()
{
#if WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)
	if (FSlateApplication::IsInitialized())
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(OutputLogModule::OutputLogTabName);
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(OutputLogModule::DeviceOutputLogTabName);
	}
#endif

#if WITH_EDITOR
	FEditorDelegates::BeginPIE.RemoveAll(this);
#endif
	FOutputLogStyle::Shutdown();
	OutputLogHistory.Reset();
}

const FName FOutputLogModule::ModuleName = UE_MODULE_NAME;

FOutputLogModule& FOutputLogModule::Get()
{
	return FModuleManager::Get().LoadModuleChecked<FOutputLogModule>(ModuleName);
}

FOutputLogModule* FOutputLogModule::TryGet()
{
	return FModuleManager::Get().GetModulePtr<FOutputLogModule>(ModuleName);
}

bool FOutputLogModule::ShouldHideConsole() const
{
	return OutputLogModule::bHideConsole;
}

TSharedRef<SWidget> FOutputLogModule::MakeConsoleInputBox(TSharedPtr<SMultiLineEditableTextBox>& OutExposedEditableTextBox, const FSimpleDelegate& OnCloseConsole, const FSimpleDelegate& OnConsoleCommandExecuted) const
{
	TSharedRef<SConsoleInputBox> NewConsoleInputBox =
		SNew(SConsoleInputBox)
		.Visibility(MakeAttributeLambda([](){ return FOutputLogModule::Get().ShouldHideConsole() ? EVisibility::Collapsed : EVisibility::Visible; }))
		.OnCloseConsole(OnCloseConsole)
		.OnConsoleCommandExecuted(OnConsoleCommandExecuted);

	OutExposedEditableTextBox = NewConsoleInputBox->GetEditableTextBox();
	return NewConsoleInputBox;
}

UE::OutputLog::FConsoleInputBoxCreationResult FOutputLogModule::MakeConsoleInputBox(const UE::OutputLog::FConsoleInputBoxCreationParams& InCreationParams) const
{
	// Create requested widget
	TSharedRef<SConsoleInputBox> NewConsoleInputBox = SNew(SConsoleInputBox)
		.Visibility(MakeAttributeLambda([]() { return FOutputLogModule::Get().ShouldHideConsole() ? EVisibility::Collapsed : EVisibility::Visible; }))
		.OnCloseConsole(InCreationParams.OnCloseConsole)
		.OnConsoleCommandExecuted(InCreationParams.OnConsoleCommandExecuted)
		.OnInterceptConsoleCommand(InCreationParams.OnInterceptConsoleCommand)
		.OnHistoryOverride(InCreationParams.OnHistoryOverride);

	// Prepare the results
	const UE::OutputLog::FConsoleInputBoxCreationResult Result
	{
		.ConsoleInputBox        = NewConsoleInputBox,
		.ExposedEditableTextBox = NewConsoleInputBox->GetEditableTextBox(),
	};

	return Result;
}

TSharedRef<SWidget> FOutputLogModule::MakeOutputLogDrawerWidget(const FSimpleDelegate& OnCloseConsole)
{
	TSharedPtr<SOutputLog> OutputLogDrawerPinned = OutputLogDrawer.Pin();

	if (!OutputLogDrawerPinned.IsValid())
	{
		OutputLogDrawerPinned = 
			SNew(SOutputLog, true)
			.OnCloseConsole(OnCloseConsole)
			.OnClearLog_Lambda([]
			{
				if (FOutputLogModule* OutputLogModule = FOutputLogModule::TryGet())
				{
					OutputLogModule->OnOutputLogDrawerCleared.Broadcast();
				}
			})
			.Messages(&OutputLogHistory->GetMessages())
			.InitialLogFilter(OutputLogFilterCache.Get());

		OutputLogDrawer = OutputLogDrawerPinned;
	}

	return OutputLogDrawerPinned.ToSharedRef();
}

TSharedRef<SWidget> FOutputLogModule::MakeOutputLogWidget(const FOutputLogCreationParams& Params)
{
	return SNew(SOutputLog, Params.bCreateDockInLayoutButton)
			.OnCloseConsole(Params.OnCloseConsole)
			.Messages(&OutputLogHistory->GetMessages())
			.SettingsMenuFlags(Params.SettingsMenuCreationFlags)
			.DefaultCategorySelection(Params.DefaultCategorySelection)
			.AllowInitialLogCategory(Params.AllowAsInitialLogCategory)
			.EnableLoggingLimitMenu(Params.LoggingLimit.IsSet())
			.LoggingLineLimit(Params.LoggingLimit);
}

void FOutputLogModule::SuspendGlobalLog()
{
	check(OutputLogHistory);
	OutputLogHistory->Suspend();
}


void FOutputLogModule::ResumeGlobalLog()
{
	check(OutputLogHistory);
	OutputLogHistory->Resume();
}

void FOutputLogModule::ToggleDebugConsoleForWindow(const TSharedRef<SWindow>& Window, const EDebugConsoleStyle::Type InStyle, const FDebugConsoleDelegates& DebugConsoleDelegates)
{
	if (ShouldHideConsole())
	{
		return;
	}

	bool bShouldOpen = true;
	// Close an existing console box, if there is one
	TSharedPtr< SWidget > PinnedDebugConsole(DebugConsole.Pin());
	if (PinnedDebugConsole.IsValid())
	{
		// If the console is already open close it unless it is in a different window.  In that case reopen it on that window
		bShouldOpen = false;
		TSharedPtr< SWindow > WindowForExistingConsole = FSlateApplication::Get().FindWidgetWindow(PinnedDebugConsole.ToSharedRef());
		if (WindowForExistingConsole.IsValid())
		{
			if (PreviousKeyboardFocusedWidget.IsValid())
			{
				FSlateApplication::Get().SetKeyboardFocus(PreviousKeyboardFocusedWidget.Pin());
				PreviousKeyboardFocusedWidget.Reset();
			}

			WindowForExistingConsole->RemoveOverlaySlot(PinnedDebugConsole.ToSharedRef());
			DebugConsole.Reset();
		}

		if (WindowForExistingConsole != Window)
		{
			// Console is being opened on another window
			bShouldOpen = true;
		}
	}

	TSharedPtr<SDockTab> ActiveTab = FGlobalTabmanager::Get()->GetActiveTab();
	if (ActiveTab.IsValid() && ActiveTab == OutputLogTab)
	{
		FGlobalTabmanager::Get()->DrawAttention(ActiveTab.ToSharedRef());
		bShouldOpen = false;
	}

	if (bShouldOpen)
	{
		const EDebugConsoleStyle::Type DebugConsoleStyle = InStyle;
		TSharedRef< SDebugConsole > DebugConsoleRef = SNew(SDebugConsole, DebugConsoleStyle, this, &DebugConsoleDelegates);
		DebugConsole = DebugConsoleRef;

		const int32 MaximumZOrder = MAX_int32;
		Window->AddOverlaySlot(MaximumZOrder)
			.VAlign(VAlign_Bottom)
			.HAlign(HAlign_Center)
			.Padding(10.0f)
			[
				DebugConsoleRef
			];

		PreviousKeyboardFocusedWidget = FSlateApplication::Get().GetKeyboardFocusedWidget();
	
		// Force keyboard focus
		DebugConsoleRef->SetFocusToEditableText();
	}
}

void FOutputLogModule::CloseDebugConsole()
{
	TSharedPtr< SWidget > PinnedDebugConsole(DebugConsole.Pin());

	if (PinnedDebugConsole.IsValid())
	{
		TSharedPtr< SWindow > WindowForExistingConsole = FSlateApplication::Get().FindWidgetWindow(PinnedDebugConsole.ToSharedRef());
		if (WindowForExistingConsole.IsValid())
		{
			WindowForExistingConsole->RemoveOverlaySlot(PinnedDebugConsole.ToSharedRef());
			DebugConsole.Reset();

			if (TSharedPtr<SWidget> PreviousKeyboardFocusedWidgetPinned = PreviousKeyboardFocusedWidget.Pin())
			{
				FSlateApplication::Get().SetKeyboardFocus(PreviousKeyboardFocusedWidgetPinned);
				PreviousKeyboardFocusedWidget.Reset();
			}
		}
	}
}

void FOutputLogModule::ClearOnPIE(const bool bIsSimulating)
{
	bool bClearOnPIEEnabled = false;
	GConfig->GetBool(TEXT("/Script/OutputLog.OutputLogSettings"), TEXT("bEnableOutputLogClearOnPIE"), bClearOnPIEEnabled, GEditorPerProjectIni);

	if (bClearOnPIEEnabled)
	{
		if (TSharedPtr<SOutputLog> OutputLogPinned = OutputLog.Pin())
		{
			if (OutputLogPinned->CanClearLog())
			{
				OutputLogPinned->OnClearLog();
			}
		}

		if (TSharedPtr<SOutputLog> OutputLogPinned = OutputLogDrawer.Pin())
		{
			if (OutputLogPinned->CanClearLog())
			{
				OutputLogPinned->OnClearLog();
			}
		}
	}
}

void FOutputLogModule::FocusOutputLogConsoleBox(const TSharedRef<SWidget> OutputLogToFocus)
{
	if (OutputLog == OutputLogToFocus)
	{
		OutputLog.Pin()->FocusConsoleCommandBox();
	}
	else if (OutputLogDrawer == OutputLogToFocus)
	{
		OutputLogDrawer.Pin()->FocusConsoleCommandBox();
	}
}

const TSharedPtr<SWidget> FOutputLogModule::GetOutputLog() const
{
	return OutputLog.Pin();
}

void FOutputLogModule::FocusOutputLog()
{
	// 1. Output log tab is open but not active. 
	if (OutputLog.IsValid()) 
	{
		OutputLogTab.Pin()->DrawAttention(); 
	}
#if WITH_EDITOR
	// 2. Output log tab isn't open and the window directly behind the notification window has a status bar, then open Output Log Drawer. 
	else if (GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->ActiveWindowBehindNotificationHasStatusBar())
	{
		TSharedRef<SWindow> ParentWindow = FSlateApplication::Get().GetActiveTopLevelRegularWindow().ToSharedRef(); 

		// try toggle the console to open the Output Log Drawer
		if (GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->ToggleDebugConsole(ParentWindow, true))
		{
			OutputLogDrawer.Pin()->FocusConsoleCommandBox(); 
		}
		// if unable to open the drawer, invoke a new Output Log tab. 
		else 
		{ 
			OpenOutputLog(); 
			OutputLogTab.Pin()->DrawAttention();
		}
	}
#endif
	// 3. The parent window has no status bar, then invoke a new Output Log tab.  
	else
	{
		OpenOutputLog(); 
		OutputLogTab.Pin()->DrawAttention(); 
	}
}

void FOutputLogModule::FocusOutputLogAndScrollToEnd()
{
	FocusOutputLog();

	if (TSharedPtr<SOutputLog> Log = OutputLog.Pin())
	{
		Log->OnConsoleCommandExecuted();
	}

	if (TSharedPtr<SOutputLog> Log = OutputLogDrawer.Pin())
	{
		Log->OnConsoleCommandExecuted();
	}
}

void FOutputLogModule::UpdateOutputLogFilter(const TArray<FName>& CategoriesToShow, TOptional<bool> bShowErrors, TOptional<bool> bShowWarnings, TOptional<bool> bShowLogs)
{
	FOutputFilterParams Params;
	Params.bShowErrors = bShowErrors;
	Params.bShowWarnings = bShowWarnings;
	Params.bShowLogs = bShowLogs;

	UpdateOutputLogFilter(CategoriesToShow, Params);
}

void FOutputLogModule::UpdateOutputLogFilter(const TArray<FName>& CategoriesToShow, const FOutputFilterParams& InParams)
{
	if (InParams.bShowErrors.IsSet())
	{
		OutputLogFilterCache->ErrorsFilter = InParams.bShowErrors.GetValue() ? ELogLevelFilter::Enabled : ELogLevelFilter::None;
	}
	// Update the filter cache to these new settings
	// This will be useful for the case where the OutputLog Drawer or Tab get created after this call
	if (InParams.bShowWarnings.IsSet())
	{
		OutputLogFilterCache->WarningsFilter = InParams.bShowWarnings.GetValue() ? ELogLevelFilter::Enabled : ELogLevelFilter::None;
	}
	if (InParams.bShowLogs.IsSet())
	{
		OutputLogFilterCache->MessagesFilter = InParams.bShowLogs.GetValue() ? ELogLevelFilter::Enabled : ELogLevelFilter::None;
	}
	if (InParams.IgnoreFilterVerbosities.IsSet())
	{
		// Redirect Log/Warning/Error versions to the enum variants.
		// Only using the set for compatability
		OutputLogFilterCache->IgnoreFilterVerbosities.Reset();
		for (ELogVerbosity::Type Verbosity : InParams.IgnoreFilterVerbosities.GetValue())
		{
			switch (Verbosity)
			{
			case ELogVerbosity::Log:
				OutputLogFilterCache->MessagesFilter = ELogLevelFilter::All;
				break;
			case ELogVerbosity::Warning:
				OutputLogFilterCache->WarningsFilter = ELogLevelFilter::All;
				break;
			case ELogVerbosity::Error:
				OutputLogFilterCache->ErrorsFilter = ELogLevelFilter::All;
				break;
			default:
				OutputLogFilterCache->IgnoreFilterVerbosities.Add(Verbosity);
			}
		}
	}

	if (CategoriesToShow.IsEmpty())
	{
		OutputLogFilterCache->SetAllCategoriesEnabled(true);
	}
	else
	{
		OutputLogFilterCache->SetAllCategoriesEnabled(false);
		for (const FName& CategoryToShow : CategoriesToShow)
		{
			OutputLogFilterCache->SetLogCategoryEnabled(CategoryToShow, true);
		}
	}
	
	if (TSharedPtr<SOutputLog> SharedOutputLog = OutputLog.Pin())
	{
		SharedOutputLog->UpdateOutputLogFilter(*OutputLogFilterCache);
	}

	if (TSharedPtr<SOutputLog> SharedOutputLogDrawer = OutputLogDrawer.Pin())
	{
		SharedOutputLogDrawer->UpdateOutputLogFilter(*OutputLogFilterCache);
	}
}

void FOutputLogModule::OpenOutputLog() const
{
	FGlobalTabmanager::Get()->TryInvokeTab(OutputLogModule::OutputLogTabName);
}

bool FOutputLogModule::ShouldCycleToOutputLogDrawer() const 
{
#if WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)
	return GetDefault<UOutputLogSettings>()->bCycleToOutputLogDrawer; 
#else
	return true;
#endif
}

const TUniquePtr<FNamePermissionList>& FOutputLogModule::GetConsoleCommandExecutorPermissionList() const
{
	return ConsoleCommandExecutorPermissionList;
}
