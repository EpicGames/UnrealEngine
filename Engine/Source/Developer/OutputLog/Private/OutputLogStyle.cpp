// Copyright Epic Games, Inc. All Rights Reserved.

#include "OutputLogStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateTypes.h"
#include "OutputLogSettings.h"
#include "Styling/StyleColors.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/CoreStyle.h"

FName FOutputLogStyle::StyleName("OutputLogStyle");
TUniquePtr<FOutputLogStyle> FOutputLogStyle::Inst(nullptr);

const FName& FOutputLogStyle::GetStyleSetName() const
{
	return StyleName;
}

const FOutputLogStyle& FOutputLogStyle::Get()
{
	if (!Inst.IsValid())
	{
		Inst = TUniquePtr<FOutputLogStyle>(new FOutputLogStyle);
	}
	return *(Inst.Get());
}

void FOutputLogStyle::Shutdown()
{
	Inst.Reset();
}

FOutputLogStyle::FOutputLogStyle()
	: FSlateStyleSet(StyleName)
{
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);

	SetParentStyleName(FAppStyle::GetAppStyleSetName());

	SetContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	// Output Log Window
	{
		const UOutputLogSettings* Settings = GetDefault<UOutputLogSettings>();
		
		const float LogFontSize = Settings ? static_cast<float>(Settings->LogFontSize) : 9.0f;

		FTextBlockStyle NormalText = FAppStyle::GetWidgetStyle<FTextBlockStyle>("NormalText");

		const FTextBlockStyle NormalLogText = FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Mono", LogFontSize))
			.SetColorAndOpacity(FStyleColors::Foreground)
			.SetSelectedBackgroundColor(FStyleColors::Highlight)
			.SetHighlightColor(FStyleColors::Black);

		Set("Log.Normal", NormalLogText);

		Set("Log.Command", FTextBlockStyle(NormalLogText)
			.SetColorAndOpacity(FStyleColors::AccentGreen)
		);

		Set("Log.Warning", FTextBlockStyle(NormalLogText)
			.SetColorAndOpacity(FStyleColors::Warning)
		);

		Set("Log.Error", FTextBlockStyle(NormalLogText)
			.SetColorAndOpacity(FStyleColors::Error)
		);

		Set("DebugConsole.Icon", new IMAGE_BRUSH_SVG("Starship/Common/Console", Icon16x16));
		Set("OutputLog.OpenSourceLocation", new IMAGE_BRUSH_SVG("Starship/Common/folder-open", Icon16x16));
		Set("OutputLog.ClearLog", new IMAGE_BRUSH_SVG("Starship/Common/delete-outline", Icon16x16));
	}

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FOutputLogStyle::~FOutputLogStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
