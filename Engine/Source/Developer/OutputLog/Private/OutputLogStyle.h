// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"
#include "Styling/SlateColor.h"
#include "UObject/NameTypes.h"
#include "Templates/UniquePtr.h"

class FOutputLogStyle
	: public FSlateStyleSet
{
public:
	virtual const FName& GetStyleSetName() const override;

	static const FOutputLogStyle& Get();
	static void Shutdown();
	
	~FOutputLogStyle();

private:
	FOutputLogStyle();

	void OnSettingsChanged(FName PropertyName);
	void UpdateLogTextSize();

	static FName StyleName;
	static TUniquePtr<FOutputLogStyle> Inst;
};
