// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDebugConsole.h"
#include "SlateOptMacros.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "SOutputLog.h"
#include "OutputLogStyle.h"

namespace DebugConsoleDefs
{
	// How many seconds to animate when console is summoned
	static const float IntroAnimationDuration = 0.25f;
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SDebugConsole::Construct( const FArguments& InArgs, const EDebugConsoleStyle::Type InStyle, FOutputLogModule* OutputLogModule, const FDebugConsoleDelegates* DebugConsoleDelegates )
{
	CurrentStyle = InStyle;

	TSharedPtr<SConsoleInputBox> ConsoleInputBox;

	check( OutputLogModule != NULL );
	ChildSlot
	[ 
		SNew( SVerticalBox )
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew( SVerticalBox )
			.Visibility( this, &SDebugConsole::MakeVisibleIfLogIsShown )
					
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding( 10.0f )
			[
				SNew(SBox)
				.HeightOverride( 200.0f )
				[
					SNew( SBorder )
					.BorderImage(FOutputLogStyle::Get().GetBrush( "ToolPanel.GroupBorder" ) )
					.ColorAndOpacity( this, &SDebugConsole::GetAnimatedColorAndOpacity )
					.BorderBackgroundColor( this, &SDebugConsole::GetAnimatedSlateColor )
					[
						SNew( SSpacer )
					]
				]
			]
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding( 10.0f )
		[
			SNew(SBox)
			.WidthOverride( 400.0f )
			.MaxDesiredHeight( 100.0f )
			[
				SNew( SBorder )
				.VAlign( VAlign_Center )
				.Padding( FMargin(3.0f, 3.0f) )
				.BorderImage(FOutputLogStyle::Get().GetBrush("Brushes.Panel"))
				.ColorAndOpacity( this, &SDebugConsole::GetAnimatedColorAndOpacity )
				.BorderBackgroundColor( this, &SDebugConsole::GetAnimatedSlateColor )
				[
					SAssignNew(ConsoleInputBox, SConsoleInputBox)
					.OnConsoleCommandExecuted(DebugConsoleDelegates->OnConsoleCommandExecuted)
					.OnCloseConsole(DebugConsoleDelegates->OnCloseConsole)
				]
			]
		]
	];

	EditableTextBox = ConsoleInputBox->GetEditableTextBox();

	// Kick off intro animation
	AnimCurveSequence = FCurveSequence();
	AnimCurve = AnimCurveSequence.AddCurve( 0.0f, DebugConsoleDefs::IntroAnimationDuration, ECurveEaseFunction::QuadOut );
	FlashCurve = AnimCurveSequence.AddCurve( DebugConsoleDefs::IntroAnimationDuration, .15f, ECurveEaseFunction::QuadInOut );

	AnimCurveSequence.Play(this->AsShared());
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

SDebugConsole::SDebugConsole()
	: CurrentStyle( EDebugConsoleStyle::Compact )
{
}


void SDebugConsole::SetFocusToEditableText()
{
	FSlateApplication::Get().SetKeyboardFocus( EditableTextBox.ToSharedRef(), EFocusCause::SetDirectly );
}

EVisibility SDebugConsole::MakeVisibleIfLogIsShown() const
{
	return CurrentStyle == EDebugConsoleStyle::WithLog ? EVisibility::Visible : EVisibility::Collapsed;
}


FLinearColor SDebugConsole::GetAnimatedColorAndOpacity() const
{
	return FLinearColor( 1.0f, 1.0f, 1.0f, AnimCurve.GetLerp() );
}


FSlateColor SDebugConsole::GetAnimatedSlateColor() const
{
	return FSlateColor( GetAnimatedColorAndOpacity() );
}

FSlateColor SDebugConsole::GetFlashColor() const
{
	float FlashAlpha = 1.0f - FlashCurve.GetLerp();

	if (FlashAlpha == 1.0f)
	{
		FlashAlpha = 0.0f;
	}

	return FLinearColor(1,1,1,FlashAlpha);
}
