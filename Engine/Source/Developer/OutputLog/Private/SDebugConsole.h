// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Animation/CurveHandle.h"
#include "Animation/CurveSequence.h"
#include "Styling/SlateColor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "OutputLogModule.h"

class SMultiLineEditableTextBox;

/**
 * Debug console widget, designed to be summoned on top of a viewport or window
 */
class SDebugConsole : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SDebugConsole )	
	{
	}

	SLATE_END_ARGS()

	/** Constructs this widget */
	void Construct( const FArguments& InArgs, const EDebugConsoleStyle::Type InStyle, FOutputLogModule* OutputLogModule, const FDebugConsoleDelegates* DebugConsoleDelegates );

	/** Call to set focus to this debug console's editable text box */
	void SetFocusToEditableText();

	/** Default constructor */
	SDebugConsole();


protected:
	/** Returns EVisibility::Visible if style has log being shown, otherwise VIS_COLLAPSED */
	EVisibility MakeVisibleIfLogIsShown() const;

	/** Returns a color and opacity value to use based on any current animation */
	FLinearColor GetAnimatedColorAndOpacity() const;

	/** Returns a Slate color based on any current animation (same color as GetAnimatedColorAndOpacity) */
	FSlateColor GetAnimatedSlateColor() const;

	FSlateColor GetFlashColor() const;
private:

	/** Editable text box for this debug console's input line */
	TSharedPtr< SMultiLineEditableTextBox > EditableTextBox;

	/** Current style of the debug console.  Can be changed on the fly. */
	EDebugConsoleStyle::Type CurrentStyle;

	/** Intro/outro animation curve */
	FCurveSequence AnimCurveSequence;
	FCurveHandle AnimCurve;
	FCurveHandle FlashCurve;
};

