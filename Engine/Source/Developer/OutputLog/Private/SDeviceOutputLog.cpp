// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDeviceOutputLog.h"
#include "Framework/Text/TextLayout.h"
#include "Widgets/Text/STextBlock.h"
#include "Misc/ScopeLock.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Images/SImage.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#if OUTPUTLOG_HAS_TARGET_PLATFORMS
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#endif
#include "PlatformInfo.h"
#include "OutputLogModule.h"
#include "OutputLogStyle.h"
#include "SSimpleComboButton.h"

#if OUTPUTLOG_HAS_TARGET_PLATFORMS
static bool IsSupportedPlatform(ITargetPlatform* Platform)
{
	check(Platform);
	return Platform->SupportsFeature( ETargetPlatformFeatures::DeviceOutputLog );
}
#endif


void SDeviceOutputLog::Construct( const FArguments& InArgs )
{
#if OUTPUTLOG_HAS_TARGET_PLATFORMS
	bAutoSelectDevice = InArgs._AutoSelectDevice;
#endif

	MessagesTextMarshaller = FOutputLogTextLayoutMarshaller::Create(TArray<TSharedPtr<FOutputLogMessage>>(), &Filter);

	MessagesTextBox = SNew(SMultiLineEditableTextBox)
		.Style(FOutputLogStyle::Get(), "Log.TextBox")
		.ForegroundColor(FLinearColor::Gray)
		.Marshaller(MessagesTextMarshaller)
		.IsReadOnly(true)
		.AlwaysShowScrollbars(true)
		.OnVScrollBarUserScrolled(this, &SDeviceOutputLog::OnUserScrolled)
		.ContextMenuExtender(this, &SDeviceOutputLog::ExtendTextBoxMenu);

	ChildSlot
	[
		SNew(SVerticalBox)

			// Output log area
			+SVerticalBox::Slot()
			.FillHeight(1)
			[
				MessagesTextBox.ToSharedRef()
			]
#if OUTPUTLOG_HAS_TARGET_PLATFORMS
			// The console input box
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(0.0f, 2.0f, 0.0f, 0.0f))
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SAssignNew(TargetDeviceComboButton, SComboButton)
					.ForegroundColor(FSlateColor::UseForeground())
					.OnGetMenuContent(this, &SDeviceOutputLog::MakeDeviceComboButtonMenu)
					.ContentPadding(FMargin(4.0f, 0.0f))
					.ButtonContent()
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SBox)
							.WidthOverride(16.f)
							.HeightOverride(16.f)
							[
								SNew(SImage).Image(this, &SDeviceOutputLog::GetSelectedTargetDeviceBrush)
							]
						]
			
						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Font(FOutputLogStyle::Get().GetFontStyle("NormalFontBold"))
							.Text(this, &SDeviceOutputLog::GetSelectedTargetDeviceText)
						]
					]
				]

				+SHorizontalBox::Slot()
				.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
				.FillWidth(1)
				.VAlign(VAlign_Center)
				[
					SNew(SConsoleInputBox)
					.Visibility(MakeAttributeLambda([]() { return FOutputLogModule::Get().ShouldHideConsole() ? EVisibility::Collapsed : EVisibility::Visible; }))
					.ConsoleCommandCustomExec(this, &SDeviceOutputLog::ExecuteConsoleCommand)
					.OnConsoleCommandExecuted(this, &SDeviceOutputLog::OnConsoleCommandExecuted)
					// Always place suggestions above the input line for the output log widget
					.SuggestionListPlacement( MenuPlacement_AboveAnchor )
				]
			]
#endif
	];

	bIsUserScrolled = false;
	RequestForceScroll();
	
#if OUTPUTLOG_HAS_TARGET_PLATFORMS
	ITargetPlatformControls::OnDeviceDiscovered().AddRaw(this, &SDeviceOutputLog::HandleTargetPlatformDeviceDiscovered);
	ITargetPlatformControls::OnDeviceLost().AddRaw(this, &SDeviceOutputLog::HandleTargetPlatformDeviceLost);
		
	// Get list of available devices
	for (ITargetPlatform* Platform : GetTargetPlatformManager()->GetTargetPlatforms())
	{
		if (IsSupportedPlatform(Platform))
		{
			TArray<ITargetDevicePtr> TargetDevices;
			Platform->GetAllDevices(TargetDevices);

			for (const ITargetDevicePtr& Device : TargetDevices)
			{
				if (Device.IsValid())
				{
					AddDeviceEntry(Device.ToSharedRef());
				}
			}
		}
	}
#endif
}

SDeviceOutputLog::~SDeviceOutputLog()
{
#if OUTPUTLOG_HAS_TARGET_PLATFORMS
	ITargetPlatformControls::OnDeviceDiscovered().RemoveAll(this);
	ITargetPlatformControls::OnDeviceLost().RemoveAll(this);

	// Clearing the pointer manually to ensure that when the pointed device output object is destroyed
	// SDeviceOutputLog is still in a valid state in case CurrentDeviceOutputPtr wanted to dereference it.
	CurrentDeviceOutputPtr.Reset();
#endif
}

void SDeviceOutputLog::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
#if OUTPUTLOG_HAS_TARGET_PLATFORMS
	// If auto-select is enabled request connecting to the default device and select it
	if (!CurrentDevicePtr.IsValid() && bAutoSelectDevice)
	{
		int32 DefaultDeviceEntryIdx = DeviceList.IndexOfByPredicate([this](const TSharedPtr<FTargetDeviceEntry>& Other) {
			ITargetDevicePtr PinnedPtr = Other->DeviceWeakPtr.Pin();
			return PinnedPtr->IsDefault();
		});

		if (DeviceList.IsValidIndex(DefaultDeviceEntryIdx))
		{
			ITargetDevicePtr PinnedPtr = DeviceList[DefaultDeviceEntryIdx]->DeviceWeakPtr.Pin();
			PinnedPtr->Connect();
			OnDeviceSelectionChanged(DeviceList[DefaultDeviceEntryIdx]);
		}
	}

	// If the device is selected but was not yet connected then the output router would not have been registered
	if (CurrentDevicePtr.IsValid() && !CurrentDeviceOutputPtr.IsValid())
	{
		ITargetDevicePtr PinnedPtr = CurrentDevicePtr->DeviceWeakPtr.Pin();
		if (PinnedPtr.IsValid() && PinnedPtr->IsConnected())
		{
			// It is now connected so register the output router
			CurrentDeviceOutputPtr = PinnedPtr->CreateDeviceOutputRouter(this);
		}
	}
#endif

	FScopeLock ScopeLock(&BufferedLinesSynch);
	if (BufferedLines.Num() > 0)
	{
		for (const FBufferedLine& Line : BufferedLines)
		{
			MessagesTextMarshaller->AppendPendingMessage(Line.Data.Get(), Line.Verbosity, Line.Category);
		}
		BufferedLines.Empty(32);
	}

	SOutputLog::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SDeviceOutputLog::Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category)
{
	FScopeLock ScopeLock(&BufferedLinesSynch);
	BufferedLines.Emplace(V, Category, Verbosity);
}

bool SDeviceOutputLog::CanBeUsedOnAnyThread() const
{
	return true;
}

void SDeviceOutputLog::ExecuteConsoleCommand(const FString& ExecCommand)
{
#if OUTPUTLOG_HAS_TARGET_PLATFORMS
	if (CurrentDevicePtr.IsValid())
	{
		ITargetDevicePtr PinnedPtr = CurrentDevicePtr->DeviceWeakPtr.Pin();
		if (PinnedPtr.IsValid())
		{
			PinnedPtr->ExecuteConsoleCommand(ExecCommand);
		}
	}
#endif
}

#if OUTPUTLOG_HAS_TARGET_PLATFORMS
void SDeviceOutputLog::HandleTargetPlatformDeviceLost(ITargetDeviceRef LostDevice)
{
	FTargetDeviceId LostDeviceId = LostDevice->GetId();
	
	if (CurrentDevicePtr.IsValid() && CurrentDevicePtr->DeviceId.GetDeviceName() == LostDeviceId.GetDeviceName())
	{
		// Kill device output object, but do not clean up output in the window
		CurrentDeviceOutputPtr.Reset();
	}
	
	// Should not do it, but what if someone somewhere holds strong reference to a lost device?
	for (const TSharedPtr<FTargetDeviceEntry>& EntryPtr : DeviceList)
	{
		if (EntryPtr->DeviceId.GetDeviceName() == LostDeviceId.GetDeviceName())
		{
			EntryPtr->DeviceWeakPtr = nullptr;
		}
	}
}

void SDeviceOutputLog::HandleTargetPlatformDeviceDiscovered(ITargetDeviceRef DiscoveredDevice)
{
	FTargetDeviceId DiscoveredDeviceId = DiscoveredDevice->GetId();

	int32 ExistingEntryIdx = DeviceList.IndexOfByPredicate([&](const TSharedPtr<FTargetDeviceEntry>& Other) {
		return (Other->DeviceId.GetDeviceName() == DiscoveredDeviceId.GetDeviceName());
	});

	if (DeviceList.IsValidIndex(ExistingEntryIdx))
	{
		DeviceList[ExistingEntryIdx]->DeviceWeakPtr = DiscoveredDevice;
		if (CurrentDevicePtr == DeviceList[ExistingEntryIdx])
		{
			if (!CurrentDeviceOutputPtr.IsValid())
			{
				if (CurrentDevicePtr.IsValid())
				{
					ITargetDevicePtr PinnedPtr = CurrentDevicePtr->DeviceWeakPtr.Pin();
					if (PinnedPtr.IsValid() && PinnedPtr->IsConnected())
					{
						CurrentDeviceOutputPtr = PinnedPtr->CreateDeviceOutputRouter(this);
					}
				}
			}
		}
	}
	else
	{
		AddDeviceEntry(DiscoveredDevice);
	}
}

ITargetDevicePtr SDeviceOutputLog::GetSelectedTargetDevice() const
{
	ITargetDevicePtr PinnedPtr = nullptr;
	if (CurrentDevicePtr.IsValid())
	{
		PinnedPtr = CurrentDevicePtr->DeviceWeakPtr.Pin();
	}
	return PinnedPtr;
}

void SDeviceOutputLog::AddDeviceEntry(ITargetDeviceRef TargetDevice)
{
	if (FindDeviceEntry(TargetDevice->GetId()))
	{
		return;
	}
	const FString DummyIOSDeviceName(FString::Printf(TEXT("All_iOS_On_%s"), FPlatformProcess::ComputerName()));
	const FString DummyTVOSDeviceName(FString::Printf(TEXT("All_tvOS_On_%s"), FPlatformProcess::ComputerName()));
	if (TargetDevice->GetId().GetDeviceName().Equals(DummyIOSDeviceName, ESearchCase::IgnoreCase) ||
		TargetDevice->GetId().GetDeviceName().Equals(DummyTVOSDeviceName, ESearchCase::IgnoreCase))
	{
		return;
	}
	using namespace PlatformInfo;
	FName DeviceIconStyleName = TargetDevice->GetPlatformControls().GetPlatformInfo().GetIconStyleName(EPlatformIconSize::Normal);
	
	TSharedPtr<FTargetDeviceEntry> DeviceEntry = MakeShareable(new FTargetDeviceEntry());
	
	DeviceEntry->DeviceId = TargetDevice->GetId();
	DeviceEntry->DeviceIconBrush = FOutputLogStyle::Get().GetBrush(DeviceIconStyleName);
	DeviceEntry->DeviceWeakPtr = TargetDevice;
	
	DeviceList.Add(DeviceEntry);
}

bool SDeviceOutputLog::FindDeviceEntry(FTargetDeviceId InDeviceId)
{
	// Should not do it, but what if someone somewhere holds strong reference to a lost device?
	for (const TSharedPtr<FTargetDeviceEntry>& EntryPtr : DeviceList)
	{
		if (EntryPtr->DeviceId.GetDeviceName() == InDeviceId.GetDeviceName())
		{
			return true;
		}
	}
	return false;
}

void SDeviceOutputLog::OnDeviceSelectionChanged(FTargetDeviceEntryPtr DeviceEntry)
{
	CurrentDeviceOutputPtr.Reset();
	OnClearLog();
	CurrentDevicePtr = DeviceEntry;
	
	if (DeviceEntry.IsValid())
	{
		ITargetDevicePtr PinnedPtr = DeviceEntry->DeviceWeakPtr.Pin();
		if (PinnedPtr.IsValid() && PinnedPtr->IsConnected())
		{
			CurrentDeviceOutputPtr = PinnedPtr->CreateDeviceOutputRouter(this);
		}
		OnSelectedDeviceChangedDelegate.ExecuteIfBound(GetSelectedTargetDevice());
	}
}

TSharedRef<SWidget> SDeviceOutputLog::MakeDeviceComboButtonMenu()
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);
	for (const FTargetDeviceEntryPtr& TargetDeviceEntryPtr : DeviceList)
	{
		TSharedRef<SWidget> MenuEntryWidget = GenerateWidgetForDeviceComboBox(TargetDeviceEntryPtr);
				
		MenuBuilder.AddMenuEntry(
			FUIAction(FExecuteAction::CreateSP(this, &SDeviceOutputLog::OnDeviceSelectionChanged, TargetDeviceEntryPtr)), 
			MenuEntryWidget
			);
	}
	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SDeviceOutputLog::GenerateWidgetForDeviceComboBox(const FTargetDeviceEntryPtr& DeviceEntry) const
{
	return 
		SNew(SBox)
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(24.f)
				.HeightOverride(24.f)
				[
					SNew(SImage).Image(GetTargetDeviceBrush(DeviceEntry))
				]
			]
			
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(FMargin(4.0f, 0.0f))
			[
				SNew(STextBlock).Text(this, &SDeviceOutputLog::GetTargetDeviceText, DeviceEntry)
			]
		];
}

const FSlateBrush* SDeviceOutputLog::GetTargetDeviceBrush(FTargetDeviceEntryPtr DeviceEntry) const
{
	if (DeviceEntry.IsValid())
	{
		return DeviceEntry->DeviceIconBrush;
	}
	else
	{
		return FOutputLogStyle::Get().GetBrush("Launcher.Instance_Unknown");
	}
}

const FSlateBrush* SDeviceOutputLog::GetSelectedTargetDeviceBrush() const
{
	return GetTargetDeviceBrush(CurrentDevicePtr);
}

FText SDeviceOutputLog::GetTargetDeviceText(FTargetDeviceEntryPtr DeviceEntry) const
{
	if (DeviceEntry.IsValid())
	{
		ITargetDevicePtr PinnedPtr = DeviceEntry->DeviceWeakPtr.Pin();
		if (PinnedPtr.IsValid() && PinnedPtr->IsConnected())
		{
			FString DeviceName = PinnedPtr->GetName();
			return FText::FromString(DeviceName);
		}
		else
		{
			FString DeviceName = DeviceEntry->DeviceId.GetDeviceName();
			return FText::Format(NSLOCTEXT("OutputLog", "TargetDeviceOffline", "{0} (Offline)"), FText::FromString(DeviceName));
		}
	}
	else
	{
		return NSLOCTEXT("OutputLog", "UnknownTargetDevice", "<Unknown device>");
	}
}

FText SDeviceOutputLog::GetSelectedTargetDeviceText() const
{
	return GetTargetDeviceText(CurrentDevicePtr);
}
#endif
