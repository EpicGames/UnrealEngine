// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "SOutputLog.h"

#if OUTPUTLOG_HAS_TARGET_PLATFORMS
#include "Interfaces/TargetDeviceId.h"
#include "Interfaces/ITargetDevice.h"
#endif


class SComboButton;

#if OUTPUTLOG_HAS_TARGET_PLATFORMS
struct FTargetDeviceEntry
{
	FTargetDeviceId			DeviceId;
	const FSlateBrush*		DeviceIconBrush;
	ITargetDeviceWeakPtr	DeviceWeakPtr;
};

typedef TSharedPtr<FTargetDeviceEntry> FTargetDeviceEntryPtr;

DECLARE_DELEGATE_OneParam(FSelectedTargetDeviceChangedDelegate, ITargetDevicePtr);
#endif

class SDeviceOutputLog : public SOutputLog
{
public:
	SLATE_BEGIN_ARGS(SDeviceOutputLog) 
		: _AutoSelectDevice(false)
	{}
		SLATE_ARGUMENT(bool, AutoSelectDevice)	// If the first device found should be automatically selected.
	SLATE_END_ARGS()

	/** Destructor for output log, so we can unregister from notifications */
	virtual ~SDeviceOutputLog();

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs	Declaration used by the SNew() macro to construct this widget
	 */
	void Construct( const FArguments& InArgs );

#if OUTPUTLOG_HAS_TARGET_PLATFORMS
	FSelectedTargetDeviceChangedDelegate& OnSelectedDeviceChanged() { return OnSelectedDeviceChangedDelegate; }
	ITargetDevicePtr GetSelectedTargetDevice() const;
#endif

protected:
	// SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	
	// FOutputDevice interface
	virtual void Serialize( const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category ) override;
	virtual bool CanBeUsedOnAnyThread() const;

	void ExecuteConsoleCommand(const FString& ExecCommand);

#if OUTPUTLOG_HAS_TARGET_PLATFORMS
	/** Callback for lost target devices. */
	void HandleTargetPlatformDeviceLost(ITargetDeviceRef LostDevice);
	/** Callback for discovered target devices. */
	void HandleTargetPlatformDeviceDiscovered(ITargetDeviceRef DiscoveredDevice);

	void  AddDeviceEntry(ITargetDeviceRef TargetDevice);
	bool FindDeviceEntry(FTargetDeviceId InDeviceId);

	void OnDeviceSelectionChanged(FTargetDeviceEntryPtr DeviceEntry);
	TSharedRef<SWidget> MakeDeviceComboButtonMenu();
	TSharedRef<SWidget> GenerateWidgetForDeviceComboBox(const FTargetDeviceEntryPtr& DeviceEntry) const;

	FText GetTargetDeviceText(FTargetDeviceEntryPtr DeviceEntry) const;
	FText GetSelectedTargetDeviceText() const;
	
	const FSlateBrush* GetTargetDeviceBrush(FTargetDeviceEntryPtr DeviceEntry) const;
	const FSlateBrush* GetSelectedTargetDeviceBrush() const;
#endif

private:
#if OUTPUTLOG_HAS_TARGET_PLATFORMS
	TArray<FTargetDeviceEntryPtr>			DeviceList;
	FTargetDeviceEntryPtr					CurrentDevicePtr;
	ITargetDeviceOutputPtr					CurrentDeviceOutputPtr;

	TSharedPtr<SComboButton> TargetDeviceComboButton;
#endif
	/** Synchronization object for access to buffered lines */
	FCriticalSection		BufferedLinesSynch;
	TArray<FBufferedLine>	BufferedLines;

#if OUTPUTLOG_HAS_TARGET_PLATFORMS
	FSelectedTargetDeviceChangedDelegate OnSelectedDeviceChangedDelegate;

	bool bAutoSelectDevice;
#endif
};
