// Copyright Epic Games, Inc. All Rights Reserved.

#include "SOutputLog.h"

#include "Algo/BinarySearch.h"
#include "ConsoleSettings.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/TextLayout.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/OutputDevice.h"
#include "Misc/ScopeLock.h"
#include "SlateOptMacros.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Text/SlateTextLayout.h"
#include "Framework/Text/SlateTextRun.h"
#include "Framework/Application/SlateApplication.h"
#include "Internationalization/BreakIterator.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Features/IModularFeatures.h"
#include "Misc/CoreDelegates.h"
#include "HAL/PlatformOutputDevices.h"
#include "HAL/FileManager.h"
#include "Widgets/Input/SButton.h"
#include "Framework/Docking/TabManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "OutputLogModule.h"
#include "Widgets/Text/SlateEditableTextTypes.h"
#include "OutputLogSettings.h"
#include "OutputLogStyle.h"
#include "OutputLogMenuContext.h"
#include "ToolMenus.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Trace/SlateTrace.h"


#define LOCTEXT_NAMESPACE "SOutputLog"

namespace OutputLogUtils
{
	TArray<IConsoleCommandExecutor*> GetAllowedConsoleCommandExecutors()
	{
		TArray<IConsoleCommandExecutor*> CommandExecutors = IModularFeatures::Get().GetModularFeatureImplementations<IConsoleCommandExecutor>(IConsoleCommandExecutor::ModularFeatureName());

		const TUniquePtr<FNamePermissionList>& PermissionList = FOutputLogModule::Get().GetConsoleCommandExecutorPermissionList();
		if (PermissionList->HasFiltering())
		{
			TArray<IConsoleCommandExecutor*> FilteredCommandExecutors;
			FilteredCommandExecutors.Reserve(CommandExecutors.Num());
			for (IConsoleCommandExecutor* CommandExecutor : CommandExecutors)
			{
				if (PermissionList->PassesFilter(CommandExecutor->GetName()))
				{
					FilteredCommandExecutors.Add(CommandExecutor);
				}
			}

			return FilteredCommandExecutors;
		}

		return CommandExecutors;
	}
}

class FCategoryLineHighlighter : public ISlateLineHighlighter
{
public:
	static TSharedRef<FCategoryLineHighlighter> Create()
	{
		return MakeShareable(new FCategoryLineHighlighter());
	}

	virtual int32 OnPaint(const FPaintArgs& Args, const FTextLayout::FLineView& Line, const FVector2D Offset, const float Width, const FTextBlockStyle& DefaultStyle, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		const FVector2D Location(Line.Offset.X + Offset.X, Line.Offset.Y + Offset.Y);

		// If we've not been set to an explicit color, calculate a suitable one from the linked color
		FLinearColor SelectionBackgroundColorAndOpacity = DefaultStyle.SelectedBackgroundColor.GetColor(InWidgetStyle);// *InWidgetStyle.GetColorAndOpacityTint();
		SelectionBackgroundColorAndOpacity.A *= 0.2f;

		// The block size and offset values are pre-scaled, so we need to account for that when converting the block offsets into paint geometry
		const float InverseScale = Inverse(AllottedGeometry.Scale);

		if (Width > 0.0f)
		{
			// Draw the actual highlight rectangle
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				++LayerId,
				AllottedGeometry.ToPaintGeometry(TransformVector(InverseScale, FVector2D(Width, FMath::Max(Line.Size.Y, Line.TextHeight))), FSlateLayoutTransform(TransformPoint(InverseScale, Location))),
				&DefaultStyle.HighlightShape,
				bParentEnabled /*&& bHasKeyboardFocus*/ ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
				SelectionBackgroundColorAndOpacity
			);
		}

		return LayerId;
	}

protected:
	FCategoryLineHighlighter()
	{
	}
};


class FCategoryBadgeHighlighter : public ISlateLineHighlighter
{
public:
	static TSharedRef<FCategoryBadgeHighlighter> Create(const FLinearColor& InBadgeColor)
	{
		return MakeShareable(new FCategoryBadgeHighlighter(InBadgeColor));
	}

	virtual int32 OnPaint(const FPaintArgs& Args, const FTextLayout::FLineView& Line, const FVector2D Offset, const float Width, const FTextBlockStyle& DefaultStyle, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		const FVector2D Location(Line.Offset.X + Offset.X, Line.Offset.Y + Offset.Y);

		// The block size and offset values are pre-scaled, so we need to account for that when converting the block offsets into paint geometry
		const float InverseScale = Inverse(AllottedGeometry.Scale);

		if (Width > 0.0f)
		{
			// Draw the actual highlight rectangle
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				++LayerId,
				AllottedGeometry.ToPaintGeometry(TransformVector(InverseScale, FVector2D(Width, FMath::Max(Line.Size.Y, Line.TextHeight))), FSlateLayoutTransform(TransformPoint(InverseScale, Location))),
				&DefaultStyle.HighlightShape,
				bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
				BadgeColor
			);
		}

		return LayerId;
	}

protected:
	FLinearColor BadgeColor;

	FCategoryBadgeHighlighter(const FLinearColor& InBadgeColor)
		: BadgeColor(InBadgeColor)
	{
	}
};



/** Expression context to test the given messages against the current text filter */
class FLogFilter_TextFilterExpressionContextOutputLog : public ITextFilterExpressionContext
{
public:
	explicit FLogFilter_TextFilterExpressionContextOutputLog(const FOutputLogMessage& InMessage) : Message(&InMessage) {}

	/** Test the given value against the strings extracted from the current item */
	virtual bool TestBasicStringExpression(const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode) const override { return TextFilterUtils::TestBasicStringExpression(*Message->Message, InValue, InTextComparisonMode); }

	/**
	* Perform a complex expression test for the current item
	* No complex expressions in this case - always returns false
	*/
	virtual bool TestComplexExpression(const FName& InKey, const FTextFilterString& InValue, const ETextFilterComparisonOperation InComparisonOperation, const ETextFilterTextComparisonMode InTextComparisonMode) const override { return false; }

private:
	/** Message that is being filtered */
	const FOutputLogMessage* Message;
};

SConsoleInputBox::SConsoleInputBox()
	: bIgnoreUIUpdate(false)
	, bHasTicked(false)
{
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SConsoleInputBox::Construct(const FArguments& InArgs)
{
	OnConsoleCommandExecuted = InArgs._OnConsoleCommandExecuted;
	ConsoleCommandCustomExec = InArgs._ConsoleCommandCustomExec;
	InterceptConsoleCommand  = InArgs._OnInterceptConsoleCommand;
	HistoryOverride          = InArgs._OnHistoryOverride;
	OnCloseConsole           = InArgs._OnCloseConsole;

	if (!ConsoleCommandCustomExec.IsBound()) // custom execs always show the default executor in the UI (which has the selector disabled)
	{
		FString PreferredCommandExecutorStr;
		if (GConfig->GetString(TEXT("OutputLog"), TEXT("PreferredCommandExecutor"), PreferredCommandExecutorStr, GEditorPerProjectIni))
		{
			PreferredCommandExecutorName = *PreferredCommandExecutorStr;
		}
	}

	SyncActiveCommandExecutor();

	IModularFeatures::Get().OnModularFeatureRegistered().AddSP(this, &SConsoleInputBox::OnCommandExecutorRegistered);
	IModularFeatures::Get().OnModularFeatureUnregistered().AddSP(this, &SConsoleInputBox::OnCommandExecutorUnregistered);
	FOutputLogModule::Get().GetConsoleCommandExecutorPermissionList()->OnFilterChanged().AddSP(this, &SConsoleInputBox::SyncActiveCommandExecutor);

	EPopupMethod PopupMethod = GIsEditor ? EPopupMethod::CreateNewWindow : EPopupMethod::UseCurrentWindow;
	ChildSlot
	[
		SAssignNew( SuggestionBox, SMenuAnchor )
		.Method(PopupMethod)
		.Placement( InArgs._SuggestionListPlacement )
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(0.0f, 0.0f, 4.0f, 0.0f))
			[
				SNew(SComboButton)
				.IsEnabled(this, &SConsoleInputBox::IsCommandExecutorMenuEnabled)
				.ComboButtonStyle(FOutputLogStyle::Get(), "SimpleComboButton")
				.ContentPadding(0.f)
				.OnGetMenuContent(this, &SConsoleInputBox::GetCommandExecutorMenuContent)
				.ButtonContent()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.Padding(2.0f)
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FOutputLogStyle::Get().GetBrush("DebugConsole.Icon"))
					]
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(2.0f)
					[
						SNew(STextBlock)
						.Text(this, &SConsoleInputBox::GetActiveCommandExecutorDisplayName)
					]
				]
			]

			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SBox)
				.MinDesiredWidth(300.f)
				.MaxDesiredWidth(600.f)
				[
					SAssignNew(InputText, SMultiLineEditableTextBox)
					.Font(FOutputLogStyle::Get().GetWidgetStyle<FTextBlockStyle>("Log.Normal").Font)
					.HintText(this, &SConsoleInputBox::GetActiveCommandExecutorHintText)
					.AllowMultiLine(this, &SConsoleInputBox::GetActiveCommandExecutorAllowMultiLine)
					.OnTextCommitted(this, &SConsoleInputBox::OnTextCommitted)
					.OnTextChanged(this, &SConsoleInputBox::OnTextChanged)
					.OnKeyCharHandler(this, &SConsoleInputBox::OnKeyCharHandler)
					.OnKeyDownHandler(this, &SConsoleInputBox::OnKeyDownHandler)
					.OnIsTypedCharValid(FOnIsTypedCharValid::CreateLambda([](const TCHAR InCh) { return true; })) // allow tabs to be typed into the field
					.ClearKeyboardFocusOnCommit(false)
					.ModiferKeyForNewLine(EModifierKey::Shift)
					.ToolTipText(this, &SConsoleInputBox::GetInputHelpText)
				]
			]
		]
		.MenuContent
		(
			SNew(SBorder)
			.BorderImage(FOutputLogStyle::Get().GetBrush("Menu.Background"))
			.Padding( FMargin(2) )
			[
				SNew(SBox)
				.HeightOverride(250.f) // avoids flickering, ideally this would be adaptive to the content without flickering
				.MinDesiredWidth(300.f)
				.MaxDesiredWidth(this, &SConsoleInputBox::GetSelectionListMaxWidth)
				[
					SAssignNew(SuggestionListView, SListView< TSharedPtr<FConsoleSuggestion> >)
					.ListItemsSource(&Suggestions.SuggestionsList)
					.SelectionMode( ESelectionMode::Single )							// Ideally the mouse over would not highlight while keyboard controls the UI
					.OnGenerateRow(this, &SConsoleInputBox::MakeSuggestionListItemWidget)
					.OnSelectionChanged(this, &SConsoleInputBox::SuggestionSelectionChanged)
				]
			]
		)
	];

	// Don't let tooltips appear on top of the text box since it hampers visibility while typing the command : 
	InputText->EnableToolTipForceField(true);
	SuggestionListView->EnableToolTipForceField(true);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SConsoleInputBox::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	bHasTicked = true;

	if (!GIntraFrameDebuggingGameThread && !IsEnabled())
	{
		SetEnabled(true);
	}
	else if (GIntraFrameDebuggingGameThread && IsEnabled())
	{
		SetEnabled(false);
	}
}


void SConsoleInputBox::SuggestionSelectionChanged(TSharedPtr<FConsoleSuggestion> NewValue, ESelectInfo::Type SelectInfo)
{
	if(bIgnoreUIUpdate)
	{
		return;
	}

	Suggestions.SelectedSuggestion = Suggestions.SuggestionsList.IndexOfByPredicate([&NewValue](const TSharedPtr<FConsoleSuggestion>& InSuggestion)
	{
		return InSuggestion == NewValue;
	});

	MarkActiveSuggestion();

	// If the user selected this suggestion by clicking on it, then go ahead and close the suggestion
	// box as they've chosen the suggestion they're interested in.
	if( SelectInfo == ESelectInfo::OnMouseClick )
	{
		SuggestionBox->SetIsOpen( false );

		// Jump the caret to the end of the newly auto-completed line. This makes it so that selecting
		// an option doesn't leave the cursor in the middle of the suggestion (which makes it hard to 
		// ctrl-back out, or to type "?" for help, etc.)
		InputText->GoTo(ETextLocation::EndOfDocument);
	}

	// Ideally this would set the focus back to the edit control
//	FWidgetPath WidgetToFocusPath;
//	FSlateApplication::Get().GeneratePathToWidgetUnchecked( InputText.ToSharedRef(), WidgetToFocusPath );
//	FSlateApplication::Get().SetKeyboardFocus( WidgetToFocusPath, EFocusCause::SetDirectly );
}

FOptionalSize SConsoleInputBox::GetSelectionListMaxWidth() const
{
	// Limit the width of the suggestions list to the work area that this widget currently resides on
	const FSlateRect WidgetRect(GetCachedGeometry().GetAbsolutePosition(), GetCachedGeometry().GetAbsolutePosition() + GetCachedGeometry().GetAbsoluteSize());
	const FSlateRect WidgetWorkArea = FSlateApplication::Get().GetWorkArea(WidgetRect);
	return FMath::Max(300.0f, WidgetWorkArea.GetSize().X - 12.0f);
}

TSharedRef<ITableRow> SConsoleInputBox::MakeSuggestionListItemWidget(TSharedPtr<FConsoleSuggestion> Suggestion, const TSharedRef<STableViewBase>& OwnerTable)
{
	check(Suggestion.IsValid());

	FString SanitizedText = Suggestion->Name;
	SanitizedText.ReplaceInline(TEXT("\r\n"), TEXT("\n"), ESearchCase::CaseSensitive);
	SanitizedText.ReplaceInline(TEXT("\r"), TEXT(" "), ESearchCase::CaseSensitive);
	SanitizedText.ReplaceInline(TEXT("\n"), TEXT(" "), ESearchCase::CaseSensitive);

	return
		SNew(STableRow< TSharedPtr<FString> >, OwnerTable)
		[
			SNew(STextBlock)
			.Text(FText::FromString(SanitizedText))
			.TextStyle(FOutputLogStyle::Get(), "Log.Normal")
			.HighlightText(Suggestions.SuggestionsHighlight)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.ToolTipText(FText::FromString(Suggestion->Help))
		];
}

void SConsoleInputBox::OnTextChanged(const FText& InText)
{
	if(bIgnoreUIUpdate)
	{
		return;
	}

	const FString& InputTextStr = InputText->GetText().ToString();
	if(!InputTextStr.IsEmpty())
	{
		TArray<FConsoleSuggestion> AutoCompleteList;
		
		if (ActiveCommandExecutor)
		{
			ActiveCommandExecutor->GetSuggestedCompletions(*InputTextStr, AutoCompleteList);
		}
		else
		{
			auto OnConsoleVariable = [&AutoCompleteList](const TCHAR *Name, IConsoleObject* CVar)
			{
				if (CVar->IsEnabled())
				{
					AutoCompleteList.Add(FConsoleSuggestion(Name, CVar->GetDetailedHelp().ToString()));
				}
			};

			IConsoleManager::Get().ForEachConsoleObjectThatContains(FConsoleObjectVisitor::CreateLambda(OnConsoleVariable), *InputTextStr);
			//AutoCompleteList.Append(GetDefault<UConsoleSettings>()->GetFilteredManualAutoCompleteCommands(InputTextStr));
		}
		AutoCompleteList.Sort([InputTextStr](const FConsoleSuggestion& A, const FConsoleSuggestion& B)
		{ 
			if (A.Name.StartsWith(InputTextStr))
			{
				if (!B.Name.StartsWith(InputTextStr))
				{
					return true;
				}
			}
			else
			{
				if (B.Name.StartsWith(InputTextStr))
				{
					return false;
				}
			}

			return A.Name < B.Name;
		});


		SetSuggestions(AutoCompleteList, FText::FromString(InputTextStr));
	}
	else
	{
		ClearSuggestions();
	}
}

void SConsoleInputBox::OnTextCommitted( const FText& InText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		if (!InText.IsEmpty())
		{
			// Copy the exec text string out so we can clear the widget's contents.  If the exec command spawns
			// a new window it can cause the text box to lose focus, which will result in this function being
			// re-entered.  We want to make sure the text string is empty on re-entry, so we'll clear it out
			const FString ExecString = InText.ToString();

			bool bWasIntercepted = false;
			if (InterceptConsoleCommand.IsBound())
			{
				// Get the executor name (NAME_None represents the default console executor)
				const FName ExecutorName = ActiveCommandExecutor ? ActiveCommandExecutor->GetName() : NAME_None;
				// Execute interception logic; returns whether the command was intercepted.
				bWasIntercepted = InterceptConsoleCommand.Execute(ExecutorName, ExecString);
			}

			// Clear the console input area
			bIgnoreUIUpdate = true;
			InputText->SetText(FText::GetEmpty());
			ClearSuggestions();
			bIgnoreUIUpdate = false;

			// If not intercepted, then execute it
			if (!bWasIntercepted)
			{
				// Exec!
				if (ConsoleCommandCustomExec.IsBound())
				{
					IConsoleManager::Get().AddConsoleHistoryEntry(TEXT(""), *ExecString);
					ConsoleCommandCustomExec.Execute(ExecString);
				}
				else if (ActiveCommandExecutor)
				{
					ActiveCommandExecutor->Exec(*ExecString);
				}
			}
		}
		else
		{
			ClearSuggestions();
		}

		OnConsoleCommandExecuted.ExecuteIfBound();
	}
}

FReply SConsoleInputBox::OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& KeyEvent)
{
	if(SuggestionBox->IsOpen())
	{
		if(KeyEvent.GetKey() == EKeys::Up || KeyEvent.GetKey() == EKeys::Down)
		{
			Suggestions.StepSelectedSuggestion(KeyEvent.GetKey() == EKeys::Up ? -1 : +1);
			MarkActiveSuggestion();

			return FReply::Handled();
		}
		else if (KeyEvent.GetKey() == EKeys::Tab)
		{
			if (Suggestions.HasSuggestions())
			{
				if (Suggestions.HasSelectedSuggestion())
				{
					Suggestions.StepSelectedSuggestion(KeyEvent.IsShiftDown() ? -1 : +1);
				}
				else
				{
					Suggestions.SelectedSuggestion = 0;
				}
				MarkActiveSuggestion();
			}

			bConsumeTab = true;
			return FReply::Handled();
		}
		else if (KeyEvent.GetKey() == EKeys::Escape)
		{
			SuggestionBox->SetIsOpen(false);
			return FReply::Handled();
		}
	}
	else
	{
		const FInputChord KeyEventAsInputChord = FInputChord(KeyEvent.GetKey(), EModifierKey::FromBools(KeyEvent.IsControlDown(), KeyEvent.IsAltDown(), KeyEvent.IsShiftDown(), KeyEvent.IsCommandDown()));

		if(KeyEvent.GetKey() == EKeys::Up)
		{
			// If the command field isn't empty we need you to have pressed Control+Up to summon the history (to make sure you're not just using caret navigation)
			const bool bIsMultiLine = GetActiveCommandExecutorAllowMultiLine();
			const bool bShowHistory = InputText->GetText().IsEmpty() || KeyEvent.IsControlDown();
			if (bShowHistory)
			{
				// Generate suggestions list
				TArray<FConsoleSuggestion> History = GenerateSuggestions();

				// Set data to the suggestion widget
				SetSuggestions(History, FText::GetEmpty());
				
				if(Suggestions.HasSuggestions())
				{
					Suggestions.StepSelectedSuggestion(-1);
					MarkActiveSuggestion();
				}
			}

			// Need to always handle this for single-line controls to avoid them invoking widget navigation
			if (!bIsMultiLine || bShowHistory)
			{
				return FReply::Handled();
			}
		}
		else if (KeyEvent.GetKey() == EKeys::Escape)
		{
			if (InputText->GetText().IsEmpty())
			{
				OnCloseConsole.ExecuteIfBound();
			}
			else
			{
				// Clear the console input area
				bIgnoreUIUpdate = true;
				InputText->SetText(FText::GetEmpty());
				bIgnoreUIUpdate = false;

				ClearSuggestions();
			}

			return FReply::Handled();
		}
		else if (ActiveCommandExecutor && ActiveCommandExecutor->GetIterateExecutorHotKey() == KeyEventAsInputChord)
		{
			MakeNextCommandExecutorActive();
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

TArray<FConsoleSuggestion> SConsoleInputBox::GenerateSuggestions() const
{
	IConsoleManager& ConsoleManager = IConsoleManager::Get();
	TArray<FString> HistoryNames;

	// See if there is an external history provider
	bool bHistoryOverriden = false;
	if (HistoryOverride.IsBound())
	{
		const FName ExecutorName = ActiveCommandExecutor ? ActiveCommandExecutor->GetName() : NAME_None;
		bHistoryOverriden = HistoryOverride.Execute(ExecutorName, HistoryNames);
	}

	// If no external history provider, then fetch internally
	if (!bHistoryOverriden)
	{
		if (ActiveCommandExecutor)
		{
			ActiveCommandExecutor->GetExecHistory(HistoryNames);
		}
		else
		{
			ConsoleManager.GetConsoleHistory(TEXT(""), HistoryNames);
		}
	}

	TArray<FConsoleSuggestion> History;
	for (const FString& Name : HistoryNames)
	{
		FString HelpString;
		// Try to find a console object for this history entry in order to retrieve a help string if possible :
		const TCHAR* NamePtr = *Name;
		if (IConsoleObject* CObj = ConsoleManager.FindConsoleObject(*FParse::Token(NamePtr, /*UseEscape = */false), /*bTrackFrequentCalls = */false); CObj && CObj->IsEnabled())
		{
			HelpString = CObj->GetDetailedHelp().ToString();
		}

		History.Add(FConsoleSuggestion(Name, HelpString));
	}

	return History;
}

void SConsoleInputBox::SetSuggestions(TArray<FConsoleSuggestion>& Elements, FText Highlight)
{
	TOptional<FString> SelectionText;
	if (Suggestions.HasSelectedSuggestion())
	{
		SelectionText = Suggestions.GetSelectedSuggestion()->Name;
	}

	Suggestions.Reset();
	Suggestions.SuggestionsHighlight = Highlight;

	for(int32 i = 0; i < Elements.Num(); ++i)
	{
		Suggestions.SuggestionsList.Add(MakeShared<FConsoleSuggestion>(Elements[i]));

		if (SelectionText.IsSet() && Elements[i].Name == SelectionText.GetValue())
		{
			Suggestions.SelectedSuggestion = i;
		}
	}
	SuggestionListView->RequestListRefresh();

	if(Suggestions.HasSuggestions())
	{
		// Ideally if the selection box is open the output window is not changing it's window title (flickers)
		SuggestionBox->SetIsOpen(true, false);
		if (Suggestions.HasSelectedSuggestion())
		{
			SuggestionListView->RequestScrollIntoView(Suggestions.GetSelectedSuggestion());
		}
		else
		{
			SuggestionListView->ScrollToTop();
		}
	}
	else
	{
		SuggestionBox->SetIsOpen(false);
	}
}

void SConsoleInputBox::OnFocusLost( const FFocusEvent& InFocusEvent )
{
//	SuggestionBox->SetIsOpen(false);
}

void SConsoleInputBox::MarkActiveSuggestion()
{
	bIgnoreUIUpdate = true;
	if (Suggestions.HasSelectedSuggestion())
	{
		TSharedPtr<FConsoleSuggestion> SelectedSuggestion = Suggestions.GetSelectedSuggestion();

		SuggestionListView->SetSelection(SelectedSuggestion);
		SuggestionListView->RequestScrollIntoView(SelectedSuggestion);	// Ideally this would only scroll if outside of the view

		InputText->SetText(FText::FromString(SelectedSuggestion->Name));
	}
	else
	{
		SuggestionListView->ClearSelection();
	}
	bIgnoreUIUpdate = false;
}

void SConsoleInputBox::ClearSuggestions()
{
	SuggestionBox->SetIsOpen(false);
	Suggestions.Reset();
}

void SConsoleInputBox::OnCommandExecutorRegistered(const FName& Type, class IModularFeature* ModularFeature)
{
	if (Type == IConsoleCommandExecutor::ModularFeatureName())
	{
		SyncActiveCommandExecutor();
	}
}

void SConsoleInputBox::OnCommandExecutorUnregistered(const FName& Type, class IModularFeature* ModularFeature)
{
	if (Type == IConsoleCommandExecutor::ModularFeatureName() && ModularFeature == ActiveCommandExecutor)
	{
		SyncActiveCommandExecutor();
	}
}

void SConsoleInputBox::SyncActiveCommandExecutor()
{
	TArray<IConsoleCommandExecutor*> CommandExecutors = OutputLogUtils::GetAllowedConsoleCommandExecutors();
	CommandExecutors.Sort([](IConsoleCommandExecutor& LHS, IConsoleCommandExecutor& RHS)
	{
		return LHS.GetDisplayName().CompareTo(RHS.GetDisplayName()) < 0;
	});
	ActiveCommandExecutor = nullptr;

	if (CommandExecutors.IsValidIndex(0))
	{
		ActiveCommandExecutor = CommandExecutors[0];
	}
	// to swap to a preferred executor, try and match from the active name
	for (IConsoleCommandExecutor* CommandExecutor : CommandExecutors)
	{
		if (CommandExecutor->GetName() == PreferredCommandExecutorName)
		{
			ActiveCommandExecutor = CommandExecutor;
			break;
		}
	}
	
}

void SConsoleInputBox::SetActiveCommandExecutor(const FName InExecName)
{
	GConfig->SetString(TEXT("OutputLog"), TEXT("PreferredCommandExecutor"), *InExecName.ToString(), GEditorPerProjectIni);
	PreferredCommandExecutorName = InExecName;
	SyncActiveCommandExecutor();
}

FText SConsoleInputBox::GetActiveCommandExecutorDisplayName() const
{
	if (ActiveCommandExecutor)
	{
		return ActiveCommandExecutor->GetDisplayName();
	}
	return FText::GetEmpty();
}

FText SConsoleInputBox::GetActiveCommandExecutorHintText() const
{
	if (ActiveCommandExecutor)
	{
		return ActiveCommandExecutor->GetHintText();
	}
	return FText::GetEmpty();
}

bool SConsoleInputBox::GetActiveCommandExecutorAllowMultiLine() const
{
	if (ActiveCommandExecutor)
	{
		return ActiveCommandExecutor->AllowMultiLine();
	}
	return false;
}

FText SConsoleInputBox::GetInputHelpText() const
{
	const FString& InputTextStr = InputText->GetText().ToString();
	if (!InputTextStr.IsEmpty())
	{
		// Try to find a console object for this entry in order to retrieve a help string if possible :
		IConsoleManager& ConsoleManager = IConsoleManager::Get();
		const TCHAR* InputTextStrPtr = *InputTextStr;
		if (IConsoleObject* CObj = ConsoleManager.FindConsoleObject(*FParse::Token(InputTextStrPtr, /*UseEscape = */false), /*bTrackFrequentCalls = */false); CObj && CObj->IsEnabled())
		{
			return CObj->GetDetailedHelp();
		}
	}
	return FText::GetEmpty();
}

bool SConsoleInputBox::IsCommandExecutorMenuEnabled() const
{
	return !ConsoleCommandCustomExec.IsBound(); // custom execs always show the default executor in the UI (which has the selector disabled)
}

void SConsoleInputBox::MakeNextCommandExecutorActive()
{
	// Sorted so the iteration order matches the displayed order.
	TArray<IConsoleCommandExecutor*> CommandExecutors = OutputLogUtils::GetAllowedConsoleCommandExecutors();
	CommandExecutors.Sort([](IConsoleCommandExecutor& LHS, IConsoleCommandExecutor& RHS)
		{
			return LHS.GetDisplayName().CompareTo(RHS.GetDisplayName()) < 0;
		});

	int32 CurrentIndex = CommandExecutors.IndexOfByKey(ActiveCommandExecutor);
	if (CurrentIndex >= 0)
	{
		CurrentIndex++;
		if (CurrentIndex >= CommandExecutors.Num())
		{
			CurrentIndex = 0;
		}

		SetActiveCommandExecutor(CommandExecutors[CurrentIndex]->GetName());
	}
}

TSharedRef<SWidget> SConsoleInputBox::GetCommandExecutorMenuContent()
{
	static const FName MenuName = "OutputLog.ConsoleInputBox.CmdExecMenu";
	if (!UToolMenus::Get()->IsMenuRegistered(MenuName))
	{
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(MenuName);
		Menu->bShouldCloseWindowAfterMenuSelection = true;

		Menu->AddDynamicSection("DynamicCmdExecEntries", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			if (UConsoleInputBoxMenuContext* Context = InMenu->FindContext<UConsoleInputBoxMenuContext>())
			{
				if (TSharedPtr<SConsoleInputBox> This = Context->GetConsoleInputBox())
				{
					TArray<IConsoleCommandExecutor*> CommandExecutors = OutputLogUtils::GetAllowedConsoleCommandExecutors();
					CommandExecutors.Sort([](IConsoleCommandExecutor& LHS, IConsoleCommandExecutor& RHS)
					{
						return LHS.GetDisplayName().CompareTo(RHS.GetDisplayName()) < 0;
					});

					FToolMenuSection& Section = InMenu->AddSection("CmdExecEntries");
					for (const IConsoleCommandExecutor* CommandExecutor : CommandExecutors)
					{
						const bool bIsActiveCmdExec = This->ActiveCommandExecutor == CommandExecutor;

						Section.AddMenuEntry(
							CommandExecutor->GetName(),
							CommandExecutor->GetDisplayName(),
							CommandExecutor->GetDescription(),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateSP(This.Get(), &SConsoleInputBox::SetActiveCommandExecutor, CommandExecutor->GetName()),
								FCanExecuteAction::CreateLambda([] { return true; }),
								FIsActionChecked::CreateLambda([bIsActiveCmdExec] { return bIsActiveCmdExec; })
								),
							EUserInterfaceActionType::Check
						);
					}
				}
			}
		}));
	}

	UConsoleInputBoxMenuContext* MenuContext = NewObject<UConsoleInputBoxMenuContext>();
	MenuContext->Init(SharedThis(this));

	FToolMenuContext ToolMenuContext(MenuContext);
	return UToolMenus::Get()->GenerateWidget(MenuName, ToolMenuContext);
}

FReply SConsoleInputBox::OnKeyDownHandler(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	const FInputChord InputChord = FInputChord(InKeyEvent.GetKey(), EModifierKey::FromBools(InKeyEvent.IsControlDown(), InKeyEvent.IsAltDown(), InKeyEvent.IsShiftDown(), InKeyEvent.IsCommandDown()));

	// Intercept the "open console" key
	if (ActiveCommandExecutor && (ActiveCommandExecutor->AllowHotKeyClose() && ActiveCommandExecutor->GetHotKey() == InputChord))
	{
		SuggestionBox->SetIsOpen(false);
		OnCloseConsole.ExecuteIfBound();
		return FReply::Handled();
	}
	
	return FReply::Unhandled();
}

FReply SConsoleInputBox::OnKeyCharHandler(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent)
{
	// A printable key may be used to open the console, so consume all characters before our first Tick
	if (!bHasTicked)
	{
		return FReply::Handled();
	}

	// Intercept tab if used for auto-complete
	if (InCharacterEvent.GetCharacter() == '\t' && bConsumeTab)
	{
		bConsumeTab = false;
		return FReply::Handled();
	}

	if (InCharacterEvent.GetModifierKeys().AnyModifiersDown() && InCharacterEvent.GetCharacter() == ' ')
	{	
		// Ignore space bar + a modifier key.  It should not type a space as this is used by other keyboard shortcuts
		return FReply::Handled();
	}

	FInputChord OpenConsoleChord;
	if (ActiveCommandExecutor && ActiveCommandExecutor->AllowHotKeyClose())
	{
		OpenConsoleChord = ActiveCommandExecutor->GetHotKey();

		const uint32* KeyCode = nullptr;
		const uint32* CharCode = nullptr;
		FInputKeyManager::Get().GetCodesFromKey(OpenConsoleChord.Key, KeyCode, CharCode);
		if (CharCode == nullptr)
		{
			return FReply::Unhandled();
		}

		// Intercept the "open console" key
		if (InCharacterEvent.GetCharacter() == (TCHAR)*CharCode
			&& OpenConsoleChord.NeedsControl() == InCharacterEvent.IsControlDown()
			&& OpenConsoleChord.NeedsAlt() == InCharacterEvent.IsAltDown()
			&& OpenConsoleChord.NeedsShift() == InCharacterEvent.IsShiftDown()
			&& OpenConsoleChord.NeedsCommand() == InCharacterEvent.IsCommandDown())
		{
			return FReply::Handled();
		}
		else
		{
			return FReply::Unhandled();
		}
	}
	else
	{
		return FReply::Unhandled();
	}
}

TSharedRef< FOutputLogTextLayoutMarshaller > FOutputLogTextLayoutMarshaller::Create(TArray< TSharedPtr<FOutputLogMessage> > InMessages, FOutputLogFilter* InFilter)
{
	return MakeShareable(new FOutputLogTextLayoutMarshaller(MoveTemp(InMessages), InFilter));
}

FOutputLogTextLayoutMarshaller::~FOutputLogTextLayoutMarshaller()
{
}

void FOutputLogTextLayoutMarshaller::SetText(const FString& SourceString, FTextLayout& TargetTextLayout)
{
	TextLayout = &TargetTextLayout;
	NextPendingMessageIndex = 0;
	SubmitPendingMessages();
}

void FOutputLogTextLayoutMarshaller::GetText(FString& TargetString, const FTextLayout& SourceTextLayout)
{
	SourceTextLayout.GetAsText(TargetString);
}

bool FOutputLogTextLayoutMarshaller::AppendPendingMessage(const TCHAR* InText, const ELogVerbosity::Type InVerbosity, const FName& InCategory)
{
	// We don't want to skip adding messages, so just try to acquire the lock
	FScopeLock PendingMessagesAccess(&PendingMessagesCriticalSection);
	return SOutputLog::CreateLogMessages(InText, InVerbosity, InCategory, PendingMessages);
}

bool FOutputLogTextLayoutMarshaller::SubmitPendingMessages()
{
	// We can always submit messages next tick. So only try to lock, if not possible return.
	if (PendingMessagesCriticalSection.TryLock())
	{
		Messages.Append(MoveTemp(PendingMessages));
		PendingMessages.Reset();
		PendingMessagesCriticalSection.Unlock();
	}
	else
	{
		return false;
	}

	if (Messages.IsValidIndex(NextPendingMessageIndex))
	{
		const int32 CurrentMessagesCount = Messages.Num();

		AppendPendingMessagesToTextLayout();
		NextPendingMessageIndex = CurrentMessagesCount;
		return true;
	}

	return false;
}

float FOutputLogTextLayoutMarshaller::GetCategoryHue(FName CategoryName)
{
	if (float* pResult = CategoryHueMap.Find(CategoryName))
	{
		return *pResult;
	}
	else
	{
		FRandomStream RNG(GetTypeHash(CategoryName));
		const float Hue = (float)RNG.FRandRange(0.0, 360.0);
		CategoryHueMap.Add(CategoryName, Hue);
		return Hue;
	}
}

void FOutputLogTextLayoutMarshaller::AppendPendingMessagesToTextLayout()
{
	const int32 CurrentMessagesCount = Messages.Num();
	const int32 NumPendingMessages = CurrentMessagesCount - NextPendingMessageIndex;

	if (NumPendingMessages == 0)
	{
		return;
	}

#if UE_SLATE_TRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(FOutputLogTextLayoutMarshaller::AppendPendingMessagesToTextLayout, SlateChannel);
#endif

	if (TextLayout)
	{
		// If we were previously empty, then we'd have inserted a dummy empty line into the document
		// We need to remove this line now as it would cause the message indices to get out-of-sync with the line numbers, which would break auto-scrolling
		const bool bWasEmpty = GetNumMessages() == 0;
		if (bWasEmpty)
		{
			TextLayout->ClearLines();
		}
	}
	else
	{
		MarkMessagesCacheAsDirty();
		MakeDirty();
	}

	const ELogCategoryColorizationMode CategoryColorizationMode = GetDefault<UOutputLogSettings>()->CategoryColorizationMode;

	TArray<FTextLayout::FNewLineData> LinesToAdd;
	LinesToAdd.Reserve(NumPendingMessages);
	TArray<FTextLineHighlight> Highlights;

	int32 NumAddedMessages = 0;

	auto ComputeCategoryColor = [this](const FTextBlockStyle& OriginalStyle, const FName MessageCategory)
	{
		FTextBlockStyle Result = OriginalStyle;

		FLinearColor HSV = OriginalStyle.ColorAndOpacity.GetSpecifiedColor().LinearRGBToHSV();
		HSV.R = GetCategoryHue(MessageCategory);
		HSV.G = FMath::Max(0.4f, HSV.G);
		Result.ColorAndOpacity = HSV.HSVToLinearRGB();
		return Result;
	};

	for (int32 MessageIndex = NextPendingMessageIndex; MessageIndex < CurrentMessagesCount; ++MessageIndex)
	{
		const TSharedPtr<FOutputLogMessage>& Message = Messages[MessageIndex];
		const int32 LineIndex = TextLayout->GetLineModels().Num() + NumAddedMessages;

		if (!Message)
		{
			continue;
		}

		Filter->AddAvailableLogCategory(Message->Category);
		if (!Filter->IsMessageAllowed(Message))
		{
			continue;
		}

		++NumAddedMessages;

		const FTextBlockStyle& MessageTextStyle = FOutputLogStyle::Get().GetWidgetStyle<FTextBlockStyle>(Message->Style);

		TSharedRef<FString> LineText = Message->Message;

		TArray<TSharedRef<IRun>> Runs;


		switch (CategoryColorizationMode)
		{
		case ELogCategoryColorizationMode::None:
			Runs.Add(FSlateTextRun::Create(FRunInfo(), LineText, MessageTextStyle));
			break;
		case ELogCategoryColorizationMode::ColorizeWholeLine:
			{
				const bool bUseCategoryColor = (Message->Verbosity > ELogVerbosity::Warning);
				Runs.Add(FSlateTextRun::Create(FRunInfo(), LineText, bUseCategoryColor ? ComputeCategoryColor(MessageTextStyle, Message->Category) : MessageTextStyle));
			}
			break;
		case ELogCategoryColorizationMode::ColorizeCategoryOnly:
			{
				if (Message->CategoryStartIndex >= 0)
				{
					const int32 CategoryStartIndex = Message->CategoryStartIndex;
					const int32 CategoryStopIndex = CategoryStartIndex + (int32)Message->Category.GetStringLength() + 1;
					if (CategoryStartIndex > 0)
					{
						Runs.Add(FSlateTextRun::Create(FRunInfo(), LineText, MessageTextStyle, FTextRange(0, CategoryStartIndex)));
					}
					Runs.Add(FSlateTextRun::Create(FRunInfo(), LineText, ComputeCategoryColor(MessageTextStyle, Message->Category), FTextRange(CategoryStartIndex, CategoryStopIndex)));
					Runs.Add(FSlateTextRun::Create(FRunInfo(), LineText, MessageTextStyle, FTextRange(CategoryStopIndex, LineText->Len())));
				}
				else
				{
					Runs.Add(FSlateTextRun::Create(FRunInfo(), LineText, MessageTextStyle));
				}
			}
			break;
		case ELogCategoryColorizationMode::ColorizeCategoryAsBadge:
			{
				if (Message->CategoryStartIndex >= 0)
				{
					const int32 CategoryStartIndex = Message->CategoryStartIndex;
					const int32 CategoryStopIndex = CategoryStartIndex + (int32)Message->Category.GetStringLength();

					FTextBlockStyle BadgeStyle = ComputeCategoryColor(MessageTextStyle, Message->Category);
					Highlights.Emplace(LineIndex, FTextRange(CategoryStartIndex, CategoryStopIndex), /*Zorder=*/ -20, FCategoryBadgeHighlighter::Create(BadgeStyle.ColorAndOpacity.GetSpecifiedColor()));
					BadgeStyle.ColorAndOpacity = FLinearColor::Black;

					if (CategoryStartIndex > 0)
					{
						Runs.Add(FSlateTextRun::Create(FRunInfo(), LineText, MessageTextStyle, FTextRange(0, CategoryStartIndex)));
					}
					Runs.Add(FSlateTextRun::Create(FRunInfo(), LineText, BadgeStyle, FTextRange(CategoryStartIndex, CategoryStopIndex)));
					Runs.Add(FSlateTextRun::Create(FRunInfo(), LineText, MessageTextStyle, FTextRange(CategoryStopIndex, LineText->Len())));
				}
				else
				{
					Runs.Add(FSlateTextRun::Create(FRunInfo(), LineText, MessageTextStyle));
				}
			}
			break;
		}

		if (!Message->Category.IsNone() && (Message->Category == CategoryToHighlight))
		{
			Highlights.Emplace(LineIndex, FTextRange(0, LineText->Len()), /*Zorder=*/ -5, FCategoryLineHighlighter::Create());
		}

		LinesToAdd.Emplace(MoveTemp(LineText), MoveTemp(Runs));
	}

	// Increment the cached message count if the log is not being rebuilt
	if ( !IsDirty() )
	{
		CachedNumMessages += NumAddedMessages;
	}

	TextLayout->AddLines(LinesToAdd);

	for (const FTextLineHighlight& Highlight : Highlights)
	{
		TextLayout->AddLineHighlight(Highlight);
	}
}

void FOutputLogTextLayoutMarshaller::ClearMessages()
{
	NextPendingMessageIndex = 0;
	Messages.Empty();
	bNumMessagesCacheDirty = true;
	MakeDirty();
}

void FOutputLogTextLayoutMarshaller::CountMessages()
{
	// Do not re-count if not dirty
	if (!bNumMessagesCacheDirty)
	{
		return;
	}

	CachedNumMessages = 0;

	for (int32 MessageIndex = 0; MessageIndex < NextPendingMessageIndex; ++MessageIndex)
	{
		const TSharedPtr<FOutputLogMessage> CurrentMessage = Messages[MessageIndex];
		if (Filter->IsMessageAllowed(CurrentMessage))
		{
			CachedNumMessages++;
		}
	}

	// Cache re-built, remove dirty flag
	bNumMessagesCacheDirty = false;
}

int32 FOutputLogTextLayoutMarshaller::GetNumMessages() const
{
	const int32 NumPendingMessages = Messages.Num() - NextPendingMessageIndex;
	return Messages.Num() - NumPendingMessages;
}

int32 FOutputLogTextLayoutMarshaller::GetNumFilteredMessages()
{
	// Re-count messages if filter changed before we refresh
	if (bNumMessagesCacheDirty)
	{
		CountMessages();
	}

	return CachedNumMessages;
}

int32 FOutputLogTextLayoutMarshaller::GetNumCachedMessages()
{
	// Re-count messages if filter changed before we refresh
	if (bNumMessagesCacheDirty)
	{
		CountMessages();
	}

	return CachedNumMessages;
}

void FOutputLogTextLayoutMarshaller::MarkMessagesCacheAsDirty()
{
	bNumMessagesCacheDirty = true;
}

FName FOutputLogTextLayoutMarshaller::GetCategoryForLocation(const FTextLocation Location) const
{
	if (TextLayout)
	{
		TSharedRef<IBreakIterator> WordBreakIterator{ FBreakIterator::CreateWordBreakIterator() };

		int32 LineIndex = Location.GetLineIndex();

		// A Message may be split across multiple lines in the TextLayout, so work backwards to find the Category on the first line of the message.
		while (TextLayout->GetLineModels().IsValidIndex(LineIndex))
		{
			const FTextLayout::FLineModel& LineModel = TextLayout->GetLineModels()[LineIndex];	

			WordBreakIterator->SetStringRef(&LineModel.Text.Get());

			int32 PreviousBreak = WordBreakIterator->ResetToBeginning();
			int32 CurrentBreak = 0;

			// Iterate words starting from the beginning of the line, as the Category is one of the first words in a message.
			while ((CurrentBreak = WordBreakIterator->MoveToNext()) != INDEX_NONE)
			{
				FTextSelection Selection{ FTextLocation(LineIndex, CurrentBreak), FTextLocation(LineIndex, PreviousBreak) };

				FString SelectedText;
				TextLayout->GetSelectionAsText(SelectedText, Selection);

				FName PossibleCategory(SelectedText, FNAME_Find);

				if (!PossibleCategory.IsNone() && Filter->IsLogCategoryAvailable(PossibleCategory))
				{
					return PossibleCategory;
				}

				PreviousBreak = CurrentBreak;
			}

			WordBreakIterator->ClearString();

			LineIndex--;
		}
	}

	return NAME_None;
}

FTextLocation FOutputLogTextLayoutMarshaller::GetTextLocationAt(const FVector2D& Relative) const
{
	return TextLayout ? TextLayout->GetTextLocationAt(Relative) : FTextLocation(INDEX_NONE, INDEX_NONE);
}

FOutputLogTextLayoutMarshaller::FOutputLogTextLayoutMarshaller(TArray< TSharedPtr<FOutputLogMessage> > InMessages, FOutputLogFilter* InFilter)
	: Messages(MoveTemp(InMessages))
	, NextPendingMessageIndex(0)
	, CachedNumMessages(0)
	, bNumMessagesCacheDirty(false)
	, Filter(InFilter)
	, TextLayout(nullptr)
{
}

//////////////////////////////////////////////////////////////////////////

namespace
{
	const FName SettingsMenuName("OutputLog.SettingsMenu");

	const FName SettingsWordWrapEntryName("WordWrapEnable");
	const FName SettingsTimestampsSubMenuName("TimestampsSubMenu");
	const FName SettingsClearOnPIEEntryName("ClearOnPIE");

	const FName SettingsSeparatorName("Separator");

	const FName SettingsBrowseLogDirectoryEntryName("BrowseLogDirectory");
	const FName SettingsOpenLogExternalEntryName("OpenLogExternal");
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SOutputLog::Construct( const FArguments& InArgs, bool bCreateDrawerDockButton)
{
#if UE_SLATE_TRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(SOutputLog::Construct, SlateChannel);
#endif

	bShouldCreateDrawerDockButton = bCreateDrawerDockButton;

	if (InArgs._InitialLogFilter)
	{
		Filter = *InArgs._InitialLogFilter;
	}
	if (InArgs._InitialLogFilterSettings)
	{
		Filter.ApplySettings(*InArgs._InitialLogFilterSettings);
	}
	BuildInitialLogCategoryFilter(InArgs);

	bShouldShowLoggingLimitMenu = InArgs._EnableLoggingLimitMenu;
	bEnableLoggingLimit = InArgs._LoggingLineLimit.IsSet();
	LoggingLineLimit = InArgs._LoggingLineLimit.Get(10000);

	OnCloseConsole = InArgs._OnCloseConsole;

	MessagesTextMarshaller = FOutputLogTextLayoutMarshaller::Create(InArgs._Messages ? *InArgs._Messages : TArray< TSharedPtr<FOutputLogMessage> >{}, &Filter);

	MessagesTextBox = SNew(SMultiLineEditableTextBox)
		.Style(FOutputLogStyle::Get(), "Log.TextBox")
		.Marshaller(MessagesTextMarshaller)
		.IsReadOnly(true)
		.AlwaysShowScrollbars(true)
		.AutoWrapText(this, &SOutputLog::IsWordWrapEnabled)
		.OnVScrollBarUserScrolled(this, &SOutputLog::OnUserScrolled)
		.ContextMenuExtender(this, &SOutputLog::ExtendTextBoxMenu);

	// We take the settings bit flags passed in, and register a corresponding runtime tool menu profile.
	const FName SettingsMenuProfileName = GetSettingsMenuProfileForFlags(InArgs._SettingsMenuFlags);

	ChildSlot
	.Padding(3)
	[
		SNew(SVerticalBox)

		// Output Log Filter
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(0.0f, 4.0f, 0.0f, 4.0f))
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(0, 0, 4, 0)
			.FillWidth(.65f)
			[
				SAssignNew(FilterTextBox, SSearchBox)
				.HintText(LOCTEXT("SearchLogHint", "Search Log"))
				.OnTextChanged(this, &SOutputLog::OnFilterTextChanged)
				.OnTextCommitted(this, &SOutputLog::OnFilterTextCommitted)
				.DelayChangeNotificationsWhileTyping(true)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			[
				SNew(SComboButton)
				.ComboButtonStyle(FOutputLogStyle::Get(), "SimpleComboButton")
				.ToolTipText(LOCTEXT("AddFilterToolTip", "Add an output log filter."))
				.OnGetMenuContent(this, &SOutputLog::MakeAddFilterMenu)
				.ButtonContent()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FOutputLogStyle::Get().GetBrush("Icons.Filter"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2, 0, 0, 0)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("Filters", "Filters"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(4, 0)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("LogLineLimitReached", "Log line limit reached. Clear log to continue."))
				.ColorAndOpacity(FSlateColor(FLinearColor::Yellow))
				.Visibility(MakeAttributeLambda([this]() {
					if (!bEnableLoggingLimit || MessagesTextMarshaller->GetNumCachedMessages() < LoggingLineLimit)
					{
						return EVisibility::Hidden;
					}
					return EVisibility::Visible;
				}))
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(4, 0)
			[
				CreateDrawerDockButton()
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.AutoWidth()
			[
				SNew(SComboButton)
				.ComboButtonStyle(FOutputLogStyle::Get(), "SimpleComboButton")
				.OnGetMenuContent(this, &SOutputLog::GetSettingsMenuContent, SettingsMenuProfileName)
				.ButtonContent()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FOutputLogStyle::Get().GetBrush("Icons.Settings"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2, 0, 0, 0)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("SettingsButton", "Settings"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			]
		]

		// Output log area
		+SVerticalBox::Slot()
		.FillHeight(1)
		[
			MessagesTextBox.ToSharedRef()
		]

		// The console input box
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(ConsoleInputBox, SConsoleInputBox)
			.Visibility(MakeAttributeLambda([]() {
				return FOutputLogModule::Get().ShouldHideConsole() ? EVisibility::Collapsed : EVisibility::Visible;
			}))
			.OnConsoleCommandExecuted(this, &SOutputLog::OnConsoleCommandExecuted)
			.OnCloseConsole(OnCloseConsole)
			// Always place suggestions above the input line for the output log widget
			.SuggestionListPlacement(MenuPlacement_AboveAnchor) 
		]
	];

	GLog->AddOutputDevice(this);

#if WITH_EDITOR
	// Listen for style changes
	UOutputLogSettings* Settings = GetMutableDefault<UOutputLogSettings>();
	SettingsWatchHandle = Settings->OnSettingChanged().AddRaw(this, &SOutputLog::HandleSettingChanged);
#endif

	bIsUserScrolled = false;
	RequestForceScroll();

	OnClearLogDelegate = InArgs._OnClearLog;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

SOutputLog::~SOutputLog()
{
	if (GLog != nullptr)
	{
		GLog->RemoveOutputDevice(this);
	}

#if WITH_EDITOR
	if (UObjectInitialized() && !GExitPurge)
	{
		UOutputLogSettings* Settings = GetMutableDefault<UOutputLogSettings>();
		Settings->OnSettingChanged().Remove(SettingsWatchHandle);
	}
#endif

}

void SOutputLog::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (MessagesTextMarshaller->SubmitPendingMessages())
	{
		// Don't scroll to the bottom automatically when the user is scrolling the view or has scrolled it away from the bottom.
		RequestForceScroll(true);
	}

	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

static const FName NAME_StyleLogCommand(TEXT("Log.Command"));
static const FName NAME_StyleLogError(TEXT("Log.Error"));
static const FName NAME_StyleLogWarning(TEXT("Log.Warning"));
static const FName NAME_StyleLogNormal(TEXT("Log.Normal"));

bool SOutputLog::CreateLogMessages( const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category, TArray< TSharedPtr<FOutputLogMessage> >& OutMessages )
{
	if (Verbosity == ELogVerbosity::SetColor)
	{
		// Skip Color Events
		return false;
	}
	else
	{
		// Get the style for this message. When piping output from child processes (e.g., when cooking through the editor), we want to highlight messages
		// according to their original verbosity, so also check for "Error:" and "Warning:" substrings. This is consistent with how the build system processes logs.
		FName Style;
		if (Category == NAME_Cmd)
		{
			Style = NAME_StyleLogCommand;
		}
		else if (Verbosity == ELogVerbosity::Error || FCString::Stristr(V, TEXT("Error:")) != nullptr)
		{
			Style = NAME_StyleLogError;
		}
		else if (Verbosity == ELogVerbosity::Warning || FCString::Stristr(V, TEXT("Warning:")) != nullptr)
		{
			Style = NAME_StyleLogWarning;
		}
		else
		{
			Style = NAME_StyleLogNormal;
		}

		// Determine how to format timestamps
		static ELogTimes::Type LogTimestampMode = ELogTimes::None;
		if (UObjectInitialized() && !GExitPurge)
		{
			// Logging can happen very late during shutdown, even after the UObject system has been torn down, hence the init check above
			LogTimestampMode = GetDefault<UOutputLogSettings>()->LogTimestampMode;
		}

		const int32 OldNumMessages = OutMessages.Num();

		// handle multiline strings by breaking them apart by line
		TArray<FTextRange> LineRanges;
		FString CurrentLogDump = V;
		FTextRange::CalculateLineRangesFromString(CurrentLogDump, LineRanges);

		bool bIsFirstLineInMessage = true;
		for (const FTextRange& LineRange : LineRanges)
		{
			if (!LineRange.IsEmpty())
			{
				FString Line = CurrentLogDump.Mid(LineRange.BeginIndex, LineRange.Len());
				Line = Line.ConvertTabsToSpaces(4);

				// Hard-wrap lines to avoid them being too long
				static const int32 HardWrapLen = 600;
				for (int32 CurrentStartIndex = 0; CurrentStartIndex < Line.Len();)
				{
					int32 HardWrapLineLen = 0;
					if (bIsFirstLineInMessage)
					{
						int32 CategoryStartIndex;
						const FString MessagePrefix = FOutputDeviceHelper::FormatLogLine(Verbosity, Category, nullptr, LogTimestampMode, -1.0, /*out*/ &CategoryStartIndex);
						
						HardWrapLineLen = FMath::Min(HardWrapLen - MessagePrefix.Len(), Line.Len() - CurrentStartIndex);
						const FString HardWrapLine = Line.Mid(CurrentStartIndex, HardWrapLineLen);

						OutMessages.Add(MakeShared<FOutputLogMessage>(MakeShared<FString>(MessagePrefix + HardWrapLine), Verbosity, Category, Style, CategoryStartIndex));
					}
					else
					{
						HardWrapLineLen = FMath::Min(HardWrapLen, Line.Len() - CurrentStartIndex);
						FString HardWrapLine = Line.Mid(CurrentStartIndex, HardWrapLineLen);
						
						OutMessages.Add(MakeShared<FOutputLogMessage>(MakeShared<FString>(MoveTemp(HardWrapLine)), Verbosity, Category, Style, INDEX_NONE));
					}

					bIsFirstLineInMessage = false;
					CurrentStartIndex += HardWrapLineLen;
				}
			}
		}

		return OldNumMessages != OutMessages.Num();
	}
}

void SOutputLog::Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category)
{
	if (!bEnableLoggingLimit || MessagesTextMarshaller->GetNumCachedMessages() < LoggingLineLimit)
	{
		MessagesTextMarshaller->AppendPendingMessage(V, Verbosity, Category);
	}
}

TSharedRef<SWidget> SOutputLog::MakeLogLimitMenuItem()
{
	return SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("LimitLog", "Logging Limit"))
		]
		+SHorizontalBox::Slot()
		.FillWidth(1.f)
		[
			SNew(SSpacer)
		]
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		[
			SNew(SNumericEntryBox<int32>)
			.AllowSpin(true)
			.Justification(ETextJustify::Right)
			.MinDesiredValueWidth(100)
			.MaxSliderValue(100000)
			.OnValueChanged_Lambda([this](int32 NewValue){
				if (NewValue > 100)
				{
					LoggingLineLimit = NewValue;
				}
			})
			.Value_Lambda([this](){ return LoggingLineLimit; })
		];
}

void SOutputLog::ExtendTextBoxMenu(FMenuBuilder& Builder)
{
	FUIAction ClearOutputLogAction(
		FExecuteAction::CreateRaw( this, &SOutputLog::OnClearLog ),
		FCanExecuteAction::CreateSP( this, &SOutputLog::CanClearLog )
		);

	Builder.AddMenuEntry(
		NSLOCTEXT("OutputLog", "ClearLogLabel", "Clear Log"),
		NSLOCTEXT("OutputLog", "ClearLogTooltip", "Clears all log messages"),
		FSlateIcon(FOutputLogStyle::Get().GetStyleSetName(), "OutputLog.ClearLog"),
		ClearOutputLogAction
		);

	Builder.AddMenuEntry(
		NSLOCTEXT("OutputLog", "ShowLogInExplorerLabel", "Show Log File in Explorer"),
		NSLOCTEXT("OutputLog", "ShowLogInExplorerTooltip", "Opens the folder containing the current Output Log file in Explorer"),
		FSlateIcon(FOutputLogStyle::Get().GetStyleSetName(), "OutputLog.OpenSourceLocation"),
		FUIAction(FExecuteAction::CreateRaw(this, &SOutputLog::OpenLogFileInExplorer))
	);

	Builder.AddMenuEntry(
		FUIAction(
			FExecuteAction::CreateLambda([this](){ bEnableLoggingLimit = !bEnableLoggingLimit; }),
			FCanExecuteAction::CreateLambda([] { return true; }),
			FIsActionChecked::CreateLambda([this] { return bEnableLoggingLimit; }),
			FIsActionButtonVisible::CreateLambda([this] { return bShouldShowLoggingLimitMenu; })
		),
		MakeLogLimitMenuItem(),
		NAME_None,
		LOCTEXT("LimitLogToolTip", "Limits Logging to specified number of lines."),
		EUserInterfaceActionType::ToggleButton);

	const FVector2D CursorPos = FSlateApplication::Get().GetCursorPos();
	const FVector2D RelativeCursorPos = MessagesTextBox->GetTickSpaceGeometry().AbsoluteToLocal(CursorPos);
	const FTextLocation CursorTextLocation = MessagesTextMarshaller->GetTextLocationAt(RelativeCursorPos);

	if (CursorTextLocation.IsValid())
	{
		const FName CategoryName = MessagesTextMarshaller->GetCategoryForLocation(CursorTextLocation);

		if (!CategoryName.IsNone())
		{
			Builder.BeginSection(NAME_None, FText::Format(LOCTEXT("CategoryActionsSectionHeading", "Category {0}"), FText::FromName(CategoryName)));

			if (CategoryName == MessagesTextMarshaller->GetCategoryToHighlight())
			{
				FUIAction StopHighlightingCategoryAction(
					FExecuteAction::CreateRaw(this, &SOutputLog::OnHighlightCategory, FName())
				);

				Builder.AddMenuEntry(
					LOCTEXT("StopHighlightCategoryAction", "Remove category highlights"),
					LOCTEXT("StopHighlightCategoryActionTooltip", "Stop highlighting all messages for this category"),
					FSlateIcon(),
					StopHighlightingCategoryAction
				);
			}
			else
			{
				FUIAction HighlightCategoryAction(
					FExecuteAction::CreateRaw(this, &SOutputLog::OnHighlightCategory, CategoryName)
				);

				Builder.AddMenuEntry(
					FText::Format(LOCTEXT("HighlightCategoryAction", "Highlight category {0}"), FText::FromName(CategoryName)),
					LOCTEXT("HighlightCategoryActionTooltip", "Highlights all messages for this category"),
					FSlateIcon(),
					HighlightCategoryAction
				);
			}

			Builder.EndSection();
		}
	}
}

void SOutputLog::OnClearLog()
{
	// Make sure the cursor is back at the start of the log before we clear it
	MessagesTextBox->GoTo(FTextLocation(0));

	MessagesTextMarshaller->ClearMessages();
	MessagesTextBox->Refresh();
	bIsUserScrolled = false;

	[[maybe_unused]] bool bOnClearLogDelegateExecuted = OnClearLogDelegate.ExecuteIfBound();
}

void SOutputLog::OnHighlightCategory(FName NewCategoryToHighlight)
{
	MessagesTextMarshaller->SetCategoryToHighlight(NewCategoryToHighlight);

	RefreshAllPreservingLocation();
}

void SOutputLog::HandleSettingChanged(FName ChangedSettingName)
{
	RefreshAllPreservingLocation();
}

void SOutputLog::RefreshAllPreservingLocation()
{
	const FTextLocation LastCursorTextLocation = MessagesTextBox->GetCursorLocation();

	MessagesTextMarshaller->MarkMessagesCacheAsDirty();
	MessagesTextMarshaller->MakeDirty();
	MessagesTextBox->Refresh();

	//@TODO: Without this, the window will scroll if the last 'normally clicked location' is not on screen
	// (even with the right-click set cursor pos fix, the refresh will scroll you back to the top of the screen
	// until you left click, or to where you last left clicked otherwise if off screen; spooky...)
	// Ideally we could read the current location or fix the bug where a refresh causes a scroll
	MessagesTextBox->GoTo(LastCursorTextLocation);
}

void SOutputLog::OnUserScrolled(float ScrollOffset)
{
	bIsUserScrolled = ScrollOffset < 1.0 && !FMath::IsNearlyEqual(ScrollOffset, 1.0f);
}

bool SOutputLog::CanClearLog() const
{
	return MessagesTextMarshaller->GetNumMessages() > 0;
}

void SOutputLog::FocusConsoleCommandBox()
{
	FSlateApplication::Get().SetKeyboardFocus(ConsoleInputBox->GetEditableTextBox(), EFocusCause::SetDirectly);
}

void SOutputLog::OnConsoleCommandExecuted()
{
	// Submit pending messages when executing a command to keep the log feeling responsive to input
	MessagesTextMarshaller->SubmitPendingMessages();
	RequestForceScroll();
}

void SOutputLog::RequestForceScroll(bool bIfUserHasNotScrolledUp)
{
	if (MessagesTextMarshaller->GetNumFilteredMessages() > 0
		&& (!bIfUserHasNotScrolledUp || !bIsUserScrolled))
	{
		MessagesTextBox->ScrollTo(ETextLocation::EndOfDocument);
		bIsUserScrolled = false;
	}
}

void SOutputLog::Refresh()
{
#if UE_SLATE_TRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(SOutputLog::Refresh, SlateChannel);
#endif

	// Re-count messages if filter changed before we refresh
	MessagesTextMarshaller->CountMessages();

	MessagesTextBox->GoTo(FTextLocation(0));
	MessagesTextMarshaller->MakeDirty();
	MessagesTextBox->Refresh();
	RequestForceScroll();
}

bool SOutputLog::IsWordWrapEnabled() const
{
	const UOutputLogSettings* Settings = GetDefault<UOutputLogSettings>();
	return Settings ? Settings->bEnableOutputLogWordWrap : false;
}

void SOutputLog::SetWordWrapEnabled(ECheckBoxState InValue)
{
	const bool bWordWrapEnabled = (InValue == ECheckBoxState::Checked);
	UOutputLogSettings* Settings = GetMutableDefault<UOutputLogSettings>();
	if (Settings)
	{
		Settings->bEnableOutputLogWordWrap = bWordWrapEnabled;
		Settings->SaveConfig();
	}

	RequestForceScroll(true);
}

ELogTimes::Type SOutputLog::GetSelectedTimestampMode() 
{
	const UOutputLogSettings* Settings = GetDefault<UOutputLogSettings>();
	return Settings->LogTimestampMode;
}

bool SOutputLog::IsSelectedTimestampMode(ELogTimes::Type NewType)
{
	return GetSelectedTimestampMode() == NewType;
}

void SOutputLog::AddTimestampMenuSection(FMenuBuilder& Menu)
{

	Menu.BeginSection("LoggingTimestampSection");
	{
		const UEnum* Enum = StaticEnum<ELogTimes::Type>();

		for (int CurrentTimeStampType = 0; CurrentTimeStampType < Enum->NumEnums() - 1; CurrentTimeStampType++)
		{
			
			ELogTimes::Type TimeStampType = static_cast<ELogTimes::Type>(CurrentTimeStampType);
			FText Tooltip;

			#if WITH_EDITOR
				Tooltip = Enum->GetToolTipTextByIndex(CurrentTimeStampType);
			#endif // WITH_EDITOR
			
			Menu.AddMenuEntry(Enum->GetDisplayNameTextByIndex(CurrentTimeStampType),
				Tooltip,
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this, TimeStampType] {
						SetTimestampMode(TimeStampType);
						}),
					FCanExecuteAction::CreateLambda([] { return true; }),
					FIsActionChecked::CreateLambda([this, TimeStampType] { return IsSelectedTimestampMode(TimeStampType); })
							),
				NAME_None,
				EUserInterfaceActionType::RadioButton);

		}
	}
	Menu.EndSection();
}

void SOutputLog::SetTimestampMode(ELogTimes::Type InValue)
{
	 UOutputLogSettings* Settings = GetMutableDefault<UOutputLogSettings>();
	if (Settings)
	{
		Settings->LogTimestampMode = InValue;
		Settings->SaveConfig();
	}
	RequestForceScroll(true);
}

#if WITH_EDITOR
bool SOutputLog::IsClearOnPIEEnabled() const
{
	const UOutputLogSettings* Settings = GetDefault<UOutputLogSettings>();
	return Settings ? Settings->bEnableOutputLogClearOnPIE : false;
}

void SOutputLog::SetClearOnPIE(ECheckBoxState InValue)
{
	const bool bClearOnPIEEnabled = (InValue == ECheckBoxState::Checked);
	UOutputLogSettings* Settings = GetMutableDefault<UOutputLogSettings>();
	if (Settings)
	{
		Settings->bEnableOutputLogClearOnPIE = bClearOnPIEEnabled;
		Settings->SaveConfig();
	}
}
#endif

void SOutputLog::BuildInitialLogCategoryFilter(const FArguments& InArgs)
{
	Filter.AllowLogCategoryCallback = InArgs._AllowInitialLogCategory;

	if (InArgs._Messages && InArgs._AllowInitialLogCategory.IsBound()) // There is no need to process the log categories now if this is not bound, as adding the messages will do it for us
	{
		bool bCategoryAlreadyInSet;
		TSet<FName> CategoryNames;
		for (const TSharedPtr<FOutputLogMessage>& Message : *InArgs._Messages)
		{
			CategoryNames.Add(Message->Category, &bCategoryAlreadyInSet);
			if (!bCategoryAlreadyInSet)
			{
				const bool bIsDeselectedByDefault = !InArgs._AllowInitialLogCategory.Execute(Message->Category);
				Filter.AddAvailableLogCategory(Message->Category, bIsDeselectedByDefault ? false : TOptional<bool>());
			}
		}
	}

	for (auto DefaultCategorySelectionIt = InArgs._DefaultCategorySelection.CreateConstIterator(); DefaultCategorySelectionIt; ++DefaultCategorySelectionIt)
	{
		Filter.SetLogCategoryEnabled(DefaultCategorySelectionIt->Key, DefaultCategorySelectionIt->Value);
	}
}

void SOutputLog::OnFilterTextChanged(const FText& InFilterText)
{
	if (Filter.GetFilterText().ToString().Equals(InFilterText.ToString(), ESearchCase::CaseSensitive))
	{
		// nothing to do
		return;
	}

	// Flag the messages count as dirty
	MessagesTextMarshaller->MarkMessagesCacheAsDirty();

	// Set filter phrases
	Filter.SetFilterText(InFilterText);

	// Report possible syntax errors back to the user
	FilterTextBox->SetError(Filter.GetSyntaxErrors());

	// Repopulate the list to show only what has not been filtered out.
	Refresh();

	// Apply the new search text
	MessagesTextBox->BeginSearch(InFilterText);
}

void SOutputLog::OnFilterTextCommitted(const FText& InFilterText, ETextCommit::Type InCommitType)
{
	OnFilterTextChanged(InFilterText);
}

TSharedRef<SWidget> SOutputLog::MakeAddFilterMenu()
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);
	
	MenuBuilder.BeginSection("OutputLogVerbosityEntries", LOCTEXT("OutputLogVerbosityHeading", "Verbosity"));
	{
		const FText AllLabel = LOCTEXT("AllLabel", "All");
		const FText EnabledLabel = LOCTEXT("EnabledLabel", "Filtered");
		const FText NoneLabel = LOCTEXT("NoneLabel", "None");
		
		MenuBuilder.AddWidget(
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0)
				.VAlign(VAlign_Center)
				.Padding(12, 0, 4, 0)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Messages", "Messages"))
					.TextStyle(FAppStyle::Get(), "Menu.Label")
					.ColorAndOpacity(FLinearColor::White)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4, 0, 12, 2)
				[
					SNew(SSegmentedControl<ELogLevelFilter>)
					.Value(this, &SOutputLog::GetMessagesFilter)
					.OnValueChanged(this, &SOutputLog::OnMessagesFilterChanged)
					+ SSegmentedControl<ELogLevelFilter>::Slot(ELogLevelFilter::None)
					.Text(NoneLabel)
					.ToolTip(LOCTEXT("NoMessagesTooltip", "No messages will be shown."))
					+ SSegmentedControl<ELogLevelFilter>::Slot(ELogLevelFilter::Enabled)
					.Text(EnabledLabel)
					.ToolTip(LOCTEXT("EnabledMessagesTooltip", "Show messages from the enabled categories."))
					+ SSegmentedControl<ELogLevelFilter>::Slot(ELogLevelFilter::All)
					.Text(AllLabel)
					.ToolTip(LOCTEXT("AllMessagesTooltip", "Show all messages, ignoring whether or not the category is enabled."))
				],
			FText::GetEmpty(),
			true
		);

		MenuBuilder.AddWidget(
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0)
				.VAlign(VAlign_Center)
				.Padding(12, 0, 4, 0)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Warnings", "Warnings"))
					.TextStyle(FAppStyle::Get(), "Menu.Label")
					.ColorAndOpacity(FLinearColor::White)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4, 0, 12, 2)
				[
					SNew(SSegmentedControl<ELogLevelFilter>)
					.Value(this, &SOutputLog::GetWarningsFilter)
					.OnValueChanged(this, &SOutputLog::OnWarningsFilterChanged)
					+ SSegmentedControl<ELogLevelFilter>::Slot(ELogLevelFilter::None)
					.Text(NoneLabel)
					.ToolTip(LOCTEXT("NoWarningsTooltip", "No warnings will be shown."))
					+ SSegmentedControl<ELogLevelFilter>::Slot(ELogLevelFilter::Enabled)
					.Text(EnabledLabel)
					.ToolTip(LOCTEXT("EnabledWarningsTooltip", "Show warnings from the enabled categories."))
					+ SSegmentedControl<ELogLevelFilter>::Slot(ELogLevelFilter::All)
					.Text(AllLabel)
					.ToolTip(LOCTEXT("AllWarningsTooltip", "Show all warnings, ignoring whether or not the category is enabled."))
				],
			FText::GetEmpty(),
			true
		);

		MenuBuilder.AddWidget(
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0)
				.VAlign(VAlign_Center)
				.Padding(12, 0, 4, 0)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Errors", "Errors"))
					.TextStyle(FAppStyle::Get(), "Menu.Label")
					.ColorAndOpacity(FLinearColor::White)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4, 0, 12, 2)
				[
					SNew(SSegmentedControl<ELogLevelFilter>)
					.Value(this, &SOutputLog::GetErrorsFilter)
					.OnValueChanged(this, &SOutputLog::OnErrorsFilterChanged)
					+ SSegmentedControl<ELogLevelFilter>::Slot(ELogLevelFilter::None)
					.Text(NoneLabel)
					.ToolTip(LOCTEXT("NoErrorsTooltip", "No errors will be shown."))
					+ SSegmentedControl<ELogLevelFilter>::Slot(ELogLevelFilter::Enabled)
					.Text(EnabledLabel)
					.ToolTip(LOCTEXT("EnabledErrorsTooltip", "Show errors from the enabled categories."))
					+ SSegmentedControl<ELogLevelFilter>::Slot(ELogLevelFilter::All)
					.Text(AllLabel)
					.ToolTip(LOCTEXT("AllErrorsTooltip", "Show all errors, ignoring whether or not the category is enabled."))
				],
			FText::GetEmpty(),
			true
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("OutputLogBetterFilter", LOCTEXT("OutputLogFilterCategories", "Categories"));
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("AllFilterCategories", "Category Filters"),
			LOCTEXT("AllFilterCategoriesTooltip", "Select the log categories that are displayed."),
			FNewMenuDelegate::CreateSP(this, &SOutputLog::MakeSelectCategoriesSubMenu),
			FUIAction(FExecuteAction::CreateSP(this, &SOutputLog::CategoriesShowAll_Execute),
				FCanExecuteAction::CreateLambda([] { return true; }),
				FGetActionCheckState::CreateSP(this, &SOutputLog::CategoriesShowAll_IsChecked)),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SOutputLog::MakeSelectCategoriesSubMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("OutputLogCategoriesEntries");
	{	
		for (const FOutputLogCategorySettings& Category : Filter.GetCategoryFilters())
		{
			FString NameString = Category.Name.ToString();
			
			MenuBuilder.AddMenuEntry(
				FText::AsCultureInvariant(NameString),
				FText::Format(LOCTEXT("Category_Tooltip", "Filter the Output Log to show category: {0}"), FText::AsCultureInvariant(NameString)),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &SOutputLog::CategoriesSingle_Execute, Category.Name),
				FCanExecuteAction::CreateLambda([] { return true; }),
				FIsActionChecked::CreateSP(this, &SOutputLog::CategoriesSingle_IsChecked, Category.Name)),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}
	}
	MenuBuilder.EndSection();
}

ELogLevelFilter SOutputLog::GetMessagesFilter() const
{
	return Filter.MessagesFilter;
}

void SOutputLog::OnMessagesFilterChanged(ELogLevelFilter NewFilter)
{
	Filter.MessagesFilter = NewFilter;
	MessagesTextMarshaller->MarkMessagesCacheAsDirty();
	Refresh();
}

ELogLevelFilter SOutputLog::GetWarningsFilter() const
{
	return Filter.WarningsFilter;
}

void SOutputLog::OnWarningsFilterChanged(ELogLevelFilter NewFilter)
{
	Filter.WarningsFilter = NewFilter;
	MessagesTextMarshaller->MarkMessagesCacheAsDirty();
	Refresh();
}

ELogLevelFilter SOutputLog::GetErrorsFilter() const
{
	return Filter.ErrorsFilter;
}

void SOutputLog::OnErrorsFilterChanged(ELogLevelFilter NewFilter)
{
	Filter.ErrorsFilter = NewFilter;
	MessagesTextMarshaller->MarkMessagesCacheAsDirty();
	Refresh();
}

ECheckBoxState SOutputLog::CategoriesShowAll_IsChecked() const
{
	return Filter.AreAllCategoriesSelected();
}

bool SOutputLog::CategoriesSingle_IsChecked(FName InName) const
{
	return Filter.IsLogCategoryEnabled(InName);
}

void SOutputLog::CategoriesShowAll_Execute()
{
	const ECheckBoxState CurrentState = Filter.AreAllCategoriesSelected();
	const bool NextState = CurrentState != ECheckBoxState::Checked;

	Filter.SetAllCategoriesEnabled(NextState);
	
	// Flag the messages count as dirty
	MessagesTextMarshaller->MarkMessagesCacheAsDirty();

	Refresh();
}

void SOutputLog::CategoriesSingle_Execute(FName InName)
{
	Filter.ToggleLogCategory(InName);

	// Flag the messages count as dirty
	MessagesTextMarshaller->MarkMessagesCacheAsDirty();

	Refresh();
}

void SOutputLog::UpdateOutputLogFilter(const FOutputLogFilter& InFilter)
{
	Filter = InFilter;
	MessagesTextMarshaller->MarkMessagesCacheAsDirty();
	Refresh();
}

void SOutputLog::UpdateOutputLogFilter(const FOutputLogFilterSettings& InSettings)
{
	Filter.ApplySettings(InSettings);
	
	MessagesTextMarshaller->MarkMessagesCacheAsDirty();
	Refresh();
}

namespace
{
	template <typename TContext>
	TSharedPtr<SOutputLog> GetWidgetFromContext(const TContext& InContext)
	{
		UOutputLogMenuContext* Context = InContext.template FindContext<UOutputLogMenuContext>();
		if (!ensure(Context))
		{
			return nullptr;
		}

		TSharedPtr<SOutputLog> Widget = Context->GetOutputLog();
		ensure(Widget);
		return Widget;
	};
};

// static
void SOutputLog::RegisterSettingsMenu()
{
	// We declare the menu structure during module load, but instantiate the
	// widget much later. Because of this, predicates/actions need to "late
	// bind" to the instance, by pulling it back out of the FToolMenuContext
	// or FToolMenuSection. See GetWidgetFromContext() above.

	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ensure(ToolMenus))
	{
		return;
	}

	if (ensure(!ToolMenus->IsMenuRegistered(SettingsMenuName)))
	{
		UToolMenu* Menu = ToolMenus->RegisterMenu(SettingsMenuName);

		FToolMenuSection& Section = Menu->AddSection(NAME_None);

		RegisterSettingsMenu_WordWrap(Section);
		RegisterSettingsMenu_TimestampMode(Section);
		RegisterSettingsMenu_ClearOnPIE(Section);

		Section.AddSeparator(SettingsSeparatorName);

		RegisterSettingsMenu_BrowseLogs(Section);
		RegisterSettingsMenu_OpenLogExternal(Section);
	}
}

// static
void SOutputLog::RegisterSettingsMenu_WordWrap(FToolMenuSection& InSection)
{
	FToolUIAction WordWrapAction;
	WordWrapAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda([](const FToolMenuContext& InContext)
		{
			if (TSharedPtr<SOutputLog> This = GetWidgetFromContext(InContext); ensure(This))
			{
				This->SetWordWrapEnabled(This->IsWordWrapEnabled() ? ECheckBoxState::Unchecked : ECheckBoxState::Checked);
			}
		});
	WordWrapAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda([](const FToolMenuContext& InContext) -> ECheckBoxState
		{
			if (TSharedPtr<SOutputLog> This = GetWidgetFromContext(InContext); ensure(This))
			{
				return This->IsWordWrapEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}

			return ECheckBoxState::Unchecked;
		});

	InSection.AddMenuEntry(
		SettingsWordWrapEntryName,
		LOCTEXT("WordWrapEnabledOption", "Enable Word Wrapping"),
		LOCTEXT("WordWrapEnabledOptionToolTip", "Enable word wrapping in the Output Log."),
		FSlateIcon(),
		WordWrapAction,
		EUserInterfaceActionType::ToggleButton
	);
}

// static
void SOutputLog::RegisterSettingsMenu_TimestampMode(FToolMenuSection& InSection)
{
	InSection.AddDynamicEntry(SettingsTimestampsSubMenuName, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			FText TimestampModeTooltip;
#if WITH_EDITORONLY_DATA
			TimestampModeTooltip = UOutputLogSettings::StaticClass()->FindPropertyByName(
				GET_MEMBER_NAME_CHECKED(UOutputLogSettings, LogTimestampMode))->GetToolTipText();
#endif // WITH_EDITORONLY_DATA

			if (TSharedPtr<SOutputLog> This = GetWidgetFromContext(InSection); ensure(This))
			{
				InSection.AddSubMenu(
					SettingsTimestampsSubMenuName,
					TAttribute<FText>::CreateLambda([This]()
						{
							const UEnum* Enum = StaticEnum<ELogTimes::Type>();
							return FText::Format(LOCTEXT("TimestampsSubmenu", "Timestamp Mode: {0}"),
								Enum->GetDisplayNameTextByIndex(This->GetSelectedTimestampMode()));
						}),
					TimestampModeTooltip,
					FNewMenuDelegate::CreateSP(This.ToSharedRef(), &SOutputLog::AddTimestampMenuSection)
				);
			}
		}));
}

// static
void SOutputLog::RegisterSettingsMenu_ClearOnPIE(FToolMenuSection& InSection)
{
#if WITH_EDITOR
	FToolUIAction ClearOnPIEAction;
	ClearOnPIEAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda([](const FToolMenuContext& InContext)
		{
			if (TSharedPtr<SOutputLog> This = GetWidgetFromContext(InContext); ensure(This))
			{
				This->SetClearOnPIE(This->IsClearOnPIEEnabled() ? ECheckBoxState::Unchecked : ECheckBoxState::Checked);
			}
		});
	ClearOnPIEAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda([](const FToolMenuContext& InContext) -> ECheckBoxState
		{
			if (TSharedPtr<SOutputLog> This = GetWidgetFromContext(InContext); ensure(This))
			{
				return This->IsClearOnPIEEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}

			return ECheckBoxState::Unchecked;
		});

	InSection.AddMenuEntry(
		SettingsClearOnPIEEntryName,
		LOCTEXT("ClearOnPIE", "Clear on PIE"),
		LOCTEXT("ClearOnPIEToolTip", "Enable clearing of the Output Log on PIE startup."),
		FSlateIcon(),
		ClearOnPIEAction,
		EUserInterfaceActionType::ToggleButton
	);
#endif
}

// static
void SOutputLog::RegisterSettingsMenu_BrowseLogs(FToolMenuSection& InSection)
{
	InSection.AddMenuEntry(
		SettingsBrowseLogDirectoryEntryName,
		LOCTEXT("FindSourceFile", "Open Source Location"),
		LOCTEXT("FindSourceFileTooltip", "Opens the folder containing the source of the Output Log."),
		FSlateIcon(FOutputLogStyle::Get().GetStyleSetName(), "OutputLog.OpenSourceLocation"),
		FToolMenuExecuteAction::CreateLambda([](const FToolMenuContext& InContext)
			{
				if (TSharedPtr<SOutputLog> This = GetWidgetFromContext(InContext); ensure(This))
				{
					This->OpenLogFileInExplorer();
				}
			})
	);
}

// static
void SOutputLog::RegisterSettingsMenu_OpenLogExternal(FToolMenuSection& InSection)
{
	InSection.AddMenuEntry(
		SettingsOpenLogExternalEntryName,
		LOCTEXT("OpenInExternalEditor", "Open In External Editor"),
		LOCTEXT("OpenInExternalEditorTooltip", "Opens the Output Log in the default external editor."),
		FSlateIcon(FOutputLogStyle::Get().GetStyleSetName(), "OutputLog.OpenInExternalEditor"),
		FToolMenuExecuteAction::CreateLambda([](const FToolMenuContext& InContext)
			{
				if (TSharedPtr<SOutputLog> This = GetWidgetFromContext(InContext); ensure(This))
				{
					This->OpenLogFileInExternalEditor();
				}
			})
	);
}

FName SOutputLog::GetSettingsMenuProfileForFlags(EOutputLogSettingsMenuFlags InFlags)
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ensure(ToolMenus) || InFlags == EOutputLogSettingsMenuFlags::None)
	{
		return NAME_None;
	}

	const FName MenuProfileName = *FString::Printf(TEXT("OutputLogSettings_Flags%i"), static_cast<int32>(InFlags));
	FToolMenuProfile* FlagsProfile = ToolMenus->FindRuntimeMenuProfile(SettingsMenuName, MenuProfileName);
	if (!FlagsProfile)
	{
		FlagsProfile = ToolMenus->AddRuntimeMenuProfile(SettingsMenuName, MenuProfileName);

		const bool bSupportWordWrapping = !EnumHasAnyFlags(InFlags, EOutputLogSettingsMenuFlags::SkipEnableWordWrapping);
		const bool bSupportClearOnPie = !EnumHasAnyFlags(InFlags, EOutputLogSettingsMenuFlags::SkipClearOnPie);
		const bool bSupportBrowseLocation = !EnumHasAnyFlags(InFlags, EOutputLogSettingsMenuFlags::SkipOpenSourceButton);
		const bool bSupportExternalEditor = !EnumHasAnyFlags(InFlags, EOutputLogSettingsMenuFlags::SkipOpenInExternalEditorButton);

		const bool bNeedsSeparator = (bSupportWordWrapping || bSupportClearOnPie) && (bSupportBrowseLocation || bSupportExternalEditor);

		if (!bSupportWordWrapping)
		{
			FlagsProfile->AddEntry(SettingsWordWrapEntryName)->Visibility = ECustomizedToolMenuVisibility::Hidden;
		}

		if (!bSupportClearOnPie)
		{
			FlagsProfile->AddEntry(SettingsClearOnPIEEntryName)->Visibility = ECustomizedToolMenuVisibility::Hidden;
		}

		if (!bNeedsSeparator)
		{
			FlagsProfile->AddEntry(SettingsSeparatorName)->Visibility = ECustomizedToolMenuVisibility::Hidden;
		}

		if (!bSupportBrowseLocation)
		{
			FlagsProfile->AddEntry(SettingsBrowseLogDirectoryEntryName)->Visibility = ECustomizedToolMenuVisibility::Hidden;
		}

		if (!bSupportExternalEditor)
		{
			FlagsProfile->AddEntry(SettingsOpenLogExternalEntryName)->Visibility = ECustomizedToolMenuVisibility::Hidden;
		}
	}

	return MenuProfileName;
}

TSharedRef<SWidget> SOutputLog::GetSettingsMenuContent(FName InMenuProfileName)
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ensure(ToolMenus))
	{
		return SNullWidget::NullWidget;
	}

	FToolMenuContext MenuContext;

	UOutputLogMenuContext* OutputLogContext = NewObject<UOutputLogMenuContext>();
	OutputLogContext->Init(SharedThis(this));
	MenuContext.AddObject(OutputLogContext);

	if (InMenuProfileName != NAME_None)
	{
		UToolMenuProfileContext* ProfileContext = NewObject<UToolMenuProfileContext>();
		ProfileContext->ActiveProfiles.Add(InMenuProfileName);
		MenuContext.AddObject(ProfileContext);
	}

	return ToolMenus->GenerateWidget(SettingsMenuName, MenuContext);
}

TSharedRef<SWidget> SOutputLog::CreateDrawerDockButton()
{
	if (bShouldCreateDrawerDockButton)
	{
		return
			SNew(SButton)
			.ButtonStyle(FOutputLogStyle::Get(), "SimpleButton")
			.ToolTipText_Lambda(
				[]()
				{
					return !FOutputLogModule::Get().GetOutputLogTab().IsValid() ?
						LOCTEXT("DockInLayout_Tooltip", "Docks this output log in the current layout.\nThe drawer will still be usable as a temporary log.") :
						LOCTEXT("FocusOnDocked_Tooltip", "Close the drawer and focus on the currently docked output log.");
				})
			.ContentPadding(FMargin(1, 0))
			.OnClicked(this, &SOutputLog::OnDockInLayoutClicked)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.0, 0.0f)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FOutputLogStyle::Get().GetBrush("Icons.Layout"))
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(4.0, 0.0f)
				[
					SNew(STextBlock)
					.Text_Lambda(
					[]()
					{
						return !FOutputLogModule::Get().GetOutputLogTab().IsValid() ?
							LOCTEXT("DockInLayout", "Dock in Layout") :
							LOCTEXT("FocusOnDocked", "Focus On Docked");
					})
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			];
	}

	return SNullWidget::NullWidget;
}

void SOutputLog::OpenLogFileInExplorer()
{
	FString Path = FPaths::ConvertRelativePathToFull(FPaths::ProjectLogDir());
	if (!Path.Len() || !IFileManager::Get().DirectoryExists(*Path))
	{
		return;
	}

	FPlatformProcess::ExploreFolder(*FPaths::GetPath(Path));
}

void SOutputLog::OpenLogFileInExternalEditor()
{
	FString Path = FPaths::ConvertRelativePathToFull(FGenericPlatformOutputDevices::GetAbsoluteLogFilename());
	if (!Path.Len() || IFileManager::Get().FileSize(*Path) == INDEX_NONE)
	{
		return;
	}

	FPlatformProcess::LaunchFileInDefaultExternalApplication(*Path, NULL, ELaunchVerb::Open);
}

FReply SOutputLog::OnDockInLayoutClicked()
{
	TSharedPtr<SDockTab> DockedTab;

	// Export our settings so that the docked tab starts from the current state.
	UOutputLogSettings* Settings = GetMutableDefault<UOutputLogSettings>();
	Filter.ExportSettings(Settings->OutputLogTabFilter);
	Settings->SaveConfig();
	
	static const FName OutputLogTabName = FName("OutputLog");
	if (TSharedPtr<SDockTab> ActiveTab = FGlobalTabmanager::Get()->GetActiveTab())
	{
		if (TSharedPtr<FTabManager> TabManager = ActiveTab->GetTabManagerPtr())
		{
			DockedTab = TabManager->TryInvokeTab(OutputLogTabName);
		}
	}
	
	if (!DockedTab)
	{
		FGlobalTabmanager::Get()->TryInvokeTab(OutputLogTabName);
	}

	OnCloseConsole.ExecuteIfBound();

	return FReply::Handled();
}

ECheckBoxState FOutputLogFilter::AreAllCategoriesSelected() const
{
	int32 NumCategoriesEnabled = 0;
	for (const TPair<FName, FOutputLogCategorySettings>& Pair : Categories)
	{
		const FOutputLogCategorySettings& Category = Pair.Value;
		if (Category.bEnabled)
		{
			++NumCategoriesEnabled;
		}
		else if (NumCategoriesEnabled > 0)
		{
			return ECheckBoxState::Undetermined; // We have both enabled and disabled ones, no need to continue
		}
	}
	
	if (NumCategoriesEnabled == Categories.Num())
	{
		return ECheckBoxState::Checked;
	}
	if (NumCategoriesEnabled == 0)
	{
		return ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Undetermined;
}

void FOutputLogFilter::ApplySettings(const FOutputLogFilterSettings& InSettings)
{
	MessagesFilter = InSettings.MessagesFilter;
	WarningsFilter = InSettings.WarningsFilter;
	ErrorsFilter = InSettings.ErrorsFilter;

	bSelectNewCategories = InSettings.bSelectNewCategories;
	
	SetFilterText(InSettings.FilterText);

	if (InSettings.Categories.Num() == 0)
	{
		// This implies all _should_ be selected
		SetAllCategoriesEnabled(true);
	}
	else
	{
		// Clear and apply piecemeal so that configured logs that 
		// haven't been hit yet are still added.
		SetAllCategoriesEnabled(false);
		for (const FOutputLogCategorySettings& Category : InSettings.Categories)
		{
			SetLogCategoryEnabled(Category.Name, Category.bEnabled);
		}
	}
}

void FOutputLogFilter::ExportSettings(FOutputLogFilterSettings& OutSettings) const
{
	OutSettings.MessagesFilter = MessagesFilter;
	OutSettings.WarningsFilter = WarningsFilter;
	OutSettings.ErrorsFilter = ErrorsFilter;
	
	OutSettings.bSelectNewCategories = bSelectNewCategories;

	if (AreAllCategoriesSelected() == ECheckBoxState::Checked)
	{
		OutSettings.Categories.Empty();
	}
	else
	{
		OutSettings.Categories = GetCategoryFilters();
	}
}

bool FOutputLogFilter::IsMessageAllowed(const TSharedPtr<FOutputLogMessage>& Message) const
{
	// Filter Verbosity
	const ELogLevelFilter LevelFilter = GetMessageLevelFilter(Message);
	if (LevelFilter == ELogLevelFilter::None)
	{
		return false;
	}

	// Filter by Category
	if (!IgnoreFilterVerbosities.Contains(Message->Verbosity) && LevelFilter == ELogLevelFilter::Enabled && !IsLogCategoryEnabled(Message->Category))
	{
		return false;
	}

	// Filter search phrase
	if (!TextFilterExpressionEvaluator.TestTextFilter(FLogFilter_TextFilterExpressionContextOutputLog(*Message)))
	{
		return false;
	}

	return true;
}

ELogLevelFilter FOutputLogFilter::GetMessageLevelFilter(const TSharedPtr<FOutputLogMessage>& Message) const
{
	switch (Message->Verbosity)
	{
	case ELogVerbosity::Error:
		return ErrorsFilter;
	case ELogVerbosity::Warning:
		return WarningsFilter;
	default:
		return MessagesFilter;
	}
}

TArray<FOutputLogCategorySettings> FOutputLogFilter::GetCategoryFilters() const
{
#if UE_SLATE_TRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(FOutputLogFilter::GetCategoryFilters, SlateChannel);
#endif

	if (bCategoriesNeedSorting)
	{
		const_cast<FOutputLogFilter*>(this)->Categories.KeyStableSort(FNameLexicalLess());
		bCategoriesNeedSorting = false;
	}
	TArray<FOutputLogCategorySettings> CategoryFilters;
	Categories.GenerateValueArray(CategoryFilters);
	return CategoryFilters;
}

void FOutputLogFilter::AddAvailableLogCategory(const FName& LogCategory, const TOptional<bool>& InitiallySelected)
{
#if UE_SLATE_TRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(FOutputLogFilter::AddAvailableLogCategory, SlateChannel);
#endif

	if (!Categories.Contains(LogCategory))
	{
		Categories.Add(LogCategory, { LogCategory, InitiallySelected.Get(bSelectNewCategories) });
		bCategoriesNeedSorting = true;
	}
}

bool FOutputLogFilter::IsLogCategoryAvailable(const FName& LogCategory) const
{
	return Categories.Find(LogCategory) != nullptr;
}

void FOutputLogFilter::SetLogCategoryEnabled(const FName& LogCategory, bool bEnabled)
{
	if (FOutputLogCategorySettings* CategoryFilter = Categories.Find(LogCategory))
	{
		CategoryFilter->bEnabled = bEnabled;
	}
	else
	{
		AddAvailableLogCategory(LogCategory, bEnabled);
	}
}

void FOutputLogFilter::SetAllCategoriesEnabled(bool bEnabled)
{
	for (TPair<FName, FOutputLogCategorySettings>& Pair : Categories)
	{
		FOutputLogCategorySettings& Category = Pair.Value;
		Category.bEnabled = bEnabled;
	}
}

void FOutputLogFilter::ToggleLogCategory(const FName& LogCategory)
{
	if (FOutputLogCategorySettings* CategoryFilter = Categories.Find(LogCategory))
	{
		CategoryFilter->bEnabled = !CategoryFilter->bEnabled;
	}
	else
	{
		AddAvailableLogCategory(LogCategory, true);
	}
}

bool FOutputLogFilter::IsLogCategoryEnabled(const FName& LogCategory) const
{
	if (const FOutputLogCategorySettings* CategoryFilter = Categories.Find(LogCategory))
	{
		return CategoryFilter->bEnabled;
	}
	return false;
}

void FOutputLogFilter::ClearSelectedLogCategories()
{
	SetAllCategoriesEnabled(false);
}
#undef LOCTEXT_NAMESPACE

