// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ConsoleInputBoxParameters.h"
#include "Framework/Text/BaseTextLayoutMarshaller.h"
#include "HAL/CriticalSection.h"
#include "HAL/IConsoleManager.h"
#include "Input/Reply.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "OutputLogCreationParams.h"
#include "OutputLogSettings.h"
#include "SlateFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

class FMenuBuilder;
class FOutputLogTextLayoutMarshaller;
class FTextLayout;
struct FToolMenuSection;
class SMenuAnchor;

/**
* A single log message for the output log, holding a message and
* a style, for color and bolding of the message.
*/
struct FOutputLogMessage
{
	TSharedRef<FString> Message;
	ELogVerbosity::Type Verbosity;
	int8 CategoryStartIndex;
	FName Category;
	FName Style;

	FOutputLogMessage(const TSharedRef<FString>& NewMessage, ELogVerbosity::Type NewVerbosity, FName NewCategory, FName NewStyle, int32 InCategoryStartIndex)
		: Message(NewMessage)
		, Verbosity(NewVerbosity)
		, CategoryStartIndex((int8)InCategoryStartIndex)
		, Category(NewCategory)
		, Style(NewStyle)
	{
	}
};

/**
 * Console input box with command-completion support
 */
class SConsoleInputBox
	: public SCompoundWidget
{
	using FInterceptConsoleCommandDelegate = UE::OutputLog::FInterceptConsoleCommandDelegate;
	using FHistoryOverrideDelegate = UE::OutputLog::FHistoryOverrideDelegate;

public:
	DECLARE_DELEGATE_OneParam(FExecuteConsoleCommand, const FString& /*ExecCommand*/)

	SLATE_BEGIN_ARGS( SConsoleInputBox )
		: _SuggestionListPlacement( MenuPlacement_BelowAnchor )
		{}

		/** Where to place the suggestion list */
		SLATE_ARGUMENT( EMenuPlacement, SuggestionListPlacement )

		/** Called before executing a command; allows intercepting it. */
		SLATE_EVENT( FInterceptConsoleCommandDelegate, OnInterceptConsoleCommand)

		/** Called to gather a command history for a command executor */
		SLATE_EVENT( FHistoryOverrideDelegate, OnHistoryOverride)

		/** Custom executor for console command, will be used when bound */
		SLATE_EVENT( FExecuteConsoleCommand, ConsoleCommandCustomExec)

		/** Called when a console command is executed */
		SLATE_EVENT( FSimpleDelegate, OnConsoleCommandExecuted )

		/** Delegate to call to close the console */
		SLATE_EVENT( FSimpleDelegate, OnCloseConsole )
	SLATE_END_ARGS()

	/** Protected console input box widget constructor, called by Slate */
	SConsoleInputBox();

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs	Declaration used by the SNew() macro to construct this widget
	 */
	void Construct( const FArguments& InArgs );

	/** Returns the editable text box associated with this widget.  Used to set focus directly. */
	TSharedRef< SMultiLineEditableTextBox > GetEditableTextBox()
	{
		return InputText.ToSharedRef();
	}

	/** SWidget interface */
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

protected:

	virtual bool SupportsKeyboardFocus() const override { return true; }

	// e.g. Tab or Key_Up
	virtual FReply OnPreviewKeyDown( const FGeometry& MyGeometry, const FKeyEvent& KeyEvent ) override;

	void OnFocusLost( const FFocusEvent& InFocusEvent ) override;

	/** Handles entering in a command */
	void OnTextCommitted(const FText& InText, ETextCommit::Type CommitInfo);

	void OnTextChanged(const FText& InText);

	/** Get the maximum width of the selection list */
	FOptionalSize GetSelectionListMaxWidth() const;

	/** Makes the widget for the suggestions messages in the list view */
	TSharedRef<ITableRow> MakeSuggestionListItemWidget(TSharedPtr<FConsoleSuggestion> Message, const TSharedRef<STableViewBase>& OwnerTable);

	void SuggestionSelectionChanged(TSharedPtr<FConsoleSuggestion> NewValue, ESelectInfo::Type SelectInfo);

	/** Genearates suggestions for the console */
	TArray<FConsoleSuggestion> GenerateSuggestions() const;

	void SetSuggestions(TArray<FConsoleSuggestion>& Elements, FText Highlight);

	void MarkActiveSuggestion();

	void ClearSuggestions();

	void OnCommandExecutorRegistered(const FName& Type, class IModularFeature* ModularFeature);

	void OnCommandExecutorUnregistered(const FName& Type, class IModularFeature* ModularFeature);

	void SyncActiveCommandExecutor();

	void SetActiveCommandExecutor(const FName InExecName);

	void MakeNextCommandExecutorActive();

	FText GetActiveCommandExecutorDisplayName() const;

	FText GetActiveCommandExecutorHintText() const;

	bool GetActiveCommandExecutorAllowMultiLine() const;

	FText GetInputHelpText() const;

	bool IsCommandExecutorMenuEnabled() const;

	TSharedRef<SWidget> GetCommandExecutorMenuContent();

	FReply OnKeyDownHandler(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);

	FReply OnKeyCharHandler(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent);

private:

	struct FSuggestions
	{
		FSuggestions()
			: SelectedSuggestion(INDEX_NONE)
		{
		}

		void Reset()
		{
			SelectedSuggestion = INDEX_NONE;
			SuggestionsList.Reset();
			SuggestionsHighlight = FText::GetEmpty();
		}

		bool HasSuggestions() const
		{
			return SuggestionsList.Num() > 0;
		}

		bool HasSelectedSuggestion() const
		{
			return SuggestionsList.IsValidIndex(SelectedSuggestion);
		}

		void StepSelectedSuggestion(const int32 Step)
		{
			SelectedSuggestion += Step;
			if (SelectedSuggestion < 0)
			{
				SelectedSuggestion = SuggestionsList.Num() - 1;
			}
			else if (SelectedSuggestion >= SuggestionsList.Num())
			{
				SelectedSuggestion = 0;
			}
		}

		TSharedPtr<FConsoleSuggestion> GetSelectedSuggestion() const
		{
			return SuggestionsList.IsValidIndex(SelectedSuggestion) ? SuggestionsList[SelectedSuggestion] : nullptr;
		}

		/** INDEX_NONE if not set, otherwise index into SuggestionsList */
		int32 SelectedSuggestion;

		/** All log messages stored in this widget for the list view */
		TArray<TSharedPtr<FConsoleSuggestion>> SuggestionsList;

		/** Highlight text to use for the suggestions list */
		FText SuggestionsHighlight;
	};

	/** Editable text widget */
	TSharedPtr< SMultiLineEditableTextBox > InputText;

	/** history / auto completion elements */
	TSharedPtr< SMenuAnchor > SuggestionBox;

	/** The list view for showing all log messages. Should be replaced by a full text editor */
	TSharedPtr< SListView< TSharedPtr<FConsoleSuggestion> > > SuggestionListView;

	/** Active list of suggestions */
	FSuggestions Suggestions;

	/** Delegate to call when a console command is executed */
	FSimpleDelegate OnConsoleCommandExecuted;

	/** Delegate used to intercept console commands. */
	FInterceptConsoleCommandDelegate InterceptConsoleCommand;

	/** Delegate used to override the command history of the specified executors. */
	FHistoryOverrideDelegate HistoryOverride;

	/** Delegate to call to execute console command */
	FExecuteConsoleCommand ConsoleCommandCustomExec;

	/** Delegate to call to close the console */
	FSimpleDelegate OnCloseConsole;

	/** Name of the preferred command executor (may not always be the active executor) */
	FName PreferredCommandExecutorName;

	/** The currently active command executor */
	IConsoleCommandExecutor* ActiveCommandExecutor;

	/** to prevent recursive calls in UI callback */
	bool bIgnoreUIUpdate;

	/** true if this widget has been Ticked at least once */
	bool bHasTicked;

	/** True if we consumed a tab key in OnPreviewKeyDown, so we can ignore it in OnKeyCharHandler as well */
	bool bConsumeTab;

};

/**
* Holds information about filters_
*/
struct FOutputLogFilter
{
	/** Set of Verbosity levels that will show all regardless of category filter */
	TSet<ELogVerbosity::Type> IgnoreFilterVerbosities;

	/** If set, checks if a category is allowed to log or not */
	FAllowLogCategoryCallback AllowLogCategoryCallback;

	ELogLevelFilter MessagesFilter = ELogLevelFilter::Enabled;
	ELogLevelFilter WarningsFilter = ELogLevelFilter::Enabled;
	ELogLevelFilter ErrorsFilter = ELogLevelFilter::Enabled;

	/** Whether newly added categories should begin as selected. */
	bool bSelectNewCategories = true;

	/** Enable all filters by default */
	FOutputLogFilter() : TextFilterExpressionEvaluator(ETextFilterExpressionEvaluatorMode::BasicString)
	{
	}

	ECheckBoxState AreAllCategoriesSelected() const;

	void ApplySettings(const FOutputLogFilterSettings& InSettings);
	void ExportSettings(FOutputLogFilterSettings& OutSettings) const;
	
	/** Checks the given message against set filters */
	bool IsMessageAllowed(const TSharedPtr<FOutputLogMessage>& Message) const;

	/** Set the Text to be used as the Filter's restrictions */
	void SetFilterText(const FText& InFilterText) { TextFilterExpressionEvaluator.SetFilterText(InFilterText); }

	/** Get the Text currently being used as the Filter's restrictions */
	FText GetFilterText() const { return TextFilterExpressionEvaluator.GetFilterText(); }

	/** Returns Evaluator syntax errors (if any) */
	FText GetSyntaxErrors() const { return TextFilterExpressionEvaluator.GetFilterErrorText(); }

	/** Returns the list of category settings sorted by Name. Ends up sorting the Categories Map if needed */
	TArray<FOutputLogCategorySettings> GetCategoryFilters() const;

	/** Returns the map of categories */
	const TMap<FName, FOutputLogCategorySettings>& GetCategories() const
	{
		return Categories;
	}

	/** Adds a Log Category to the list of available categories, if it isn't already present */
	void AddAvailableLogCategory(const FName& LogCategory, const TOptional<bool>& InitiallySelected = TOptional<bool>());

	/** Returns true if the specified log category is in the list of available categories */
	bool IsLogCategoryAvailable(const FName& LogCategory) const;
	
	/** Returns true if the specified log category is enabled */
	bool IsLogCategoryEnabled(const FName& LogCategory) const;

	/** Enables or disables a log category if it is not already in that state. */
	void SetLogCategoryEnabled(const FName& LogCategory, bool bEnabled);

	void SetAllCategoriesEnabled(bool bEnabled);
	
	/** Enables or disables a Log Category in the filter */
	void ToggleLogCategory(const FName& LogCategory);

	/** Empties the list of selected log categories */
	void ClearSelectedLogCategories();
	
private:
	ELogLevelFilter GetMessageLevelFilter(const TSharedPtr<FOutputLogMessage>& Message) const;

	mutable bool bCategoriesNeedSorting = false;
	/** All categories the filter is aware of */
	TMap<FName, FOutputLogCategorySettings> Categories;

	/** Expression evaluator that can be used to perform complex text filter queries */
	FTextFilterExpressionEvaluator TextFilterExpressionEvaluator;
};

/**
 * Widget which holds a list view of logs of the program output
 * as well as a combo box for entering in new commands
 */
class SOutputLog 
	: public SCompoundWidget, public FOutputDevice
{

public:

	SLATE_BEGIN_ARGS( SOutputLog ) {}

		SLATE_EVENT(FSimpleDelegate, OnCloseConsole)

		SLATE_EVENT(FSimpleDelegate, OnClearLog)

		/** All messages captured before this log window has been created */
		SLATE_ARGUMENT_DEFAULT( const TArray< TSharedPtr<FOutputLogMessage> >*, Messages) = nullptr;

		/** The default Filter to apply to the OutputLog */
		SLATE_ARGUMENT_DEFAULT( const FOutputLogFilter*, InitialLogFilter ) = nullptr;

		/** The default FilterSettings to apply to the OutputLog */
		SLATE_ARGUMENT_DEFAULT( const FOutputLogFilterSettings*, InitialLogFilterSettings ) = nullptr;

		/**  */
		SLATE_ARGUMENT_DEFAULT( EOutputLogSettingsMenuFlags, SettingsMenuFlags ) = EOutputLogSettingsMenuFlags::None;

		/** Should this output log enable support for limiting the number of logged lines. */
		SLATE_ARGUMENT_DEFAULT( bool, EnableLoggingLimitMenu ) = false;

		/** The limit to the number of lines we output to the logging widget. */
		SLATE_ARGUMENT_DEFAULT( TOptional<int32>, LoggingLineLimit ) = {};

		SLATE_ARGUMENT( FDefaultCategorySelectionMap, DefaultCategorySelection )

		/** Used to determine the set of initially discovered log categories that should be selected */
		SLATE_EVENT( FAllowLogCategoryCallback, AllowInitialLogCategory )

	SLATE_END_ARGS()

	/** Destructor for output log, so we can unregister from notifications */
	virtual ~SOutputLog();

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs	Declaration used by the SNew() macro to construct this widget
	 */
	void Construct( const FArguments& InArgs, bool bCreateDrawerDockButton );

	// SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/**
	 * Creates FOutputLogMessage objects from FOutputDevice log callback
	 *
	 * @param	V Message text
	 * @param Verbosity Message verbosity
	 * @param Category Message category
	 * @param OutMessages Array to receive created FOutputLogMessage messages
	 * @param Filters [Optional] Filters to apply to Messages
	 *
	 * @return true if any messages have been created, false otherwise
	 */
	static bool CreateLogMessages(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category, TArray< TSharedPtr<FOutputLogMessage> >& OutMessages);

	/** Registers settings dropdown tool menu for extensibility. */
	static void RegisterSettingsMenu();

	/**
	* Called when delete all is selected
	*/
	void OnClearLog();

	/** Called when a category is selected to be highlighted */
	void OnHighlightCategory(FName NewCategoryToHighlight);

	/** Called when the editor style settings are modified */
	void HandleSettingChanged(FName ChangedSettingName);

	void RefreshAllPreservingLocation();

	/**
	 * Called to determine whether delete all is currently a valid command
	 */
	bool CanClearLog() const;

	/** Focuses the edit box where you type in console commands */
	void FocusConsoleCommandBox();

	/** Change the output log's filter. If CategoriesToShow is empty, all categories will be shown. */
	void UpdateOutputLogFilter(const FOutputLogFilter& InFilter);
	void UpdateOutputLogFilter(const FOutputLogFilterSettings& InSettings);

	const FOutputLogFilter& GetOutputLogFilter() { return Filter; }
protected:

	virtual void Serialize( const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category ) override;

protected:
	/**
	 * Extends the context menu used by the text box
	 */
	void ExtendTextBoxMenu(FMenuBuilder& Builder);
	
	/**
	 * Called when the user scrolls the log window vertically
	 */
	void OnUserScrolled(float ScrollOffset);

public:
	/** Called when a console command is entered for this output log */
	void OnConsoleCommandExecuted();

protected:
	/** Request we immediately force scroll to the bottom of the log */
	void RequestForceScroll(bool bIfUserHasNotScrolledUp = false);

	/** Converts the array of messages into something the text box understands */
	TSharedPtr< FOutputLogTextLayoutMarshaller > MessagesTextMarshaller;

	/** The editable text showing all log messages */
	TSharedPtr< SMultiLineEditableTextBox > MessagesTextBox;

	/** The editable text showing all log messages */
	TSharedPtr< SSearchBox > FilterTextBox;

	FSimpleDelegate OnClearLogDelegate; 
    FSimpleDelegate OnCloseConsole;

private:

	/** The log limit menu entry to prevent logs from consuming the output log widget. */
	TSharedRef<SWidget> MakeLogLimitMenuItem();

	void BuildInitialLogCategoryFilter(const FArguments& InArgs);
	
	/** Called by Slate when the filter box changes text. */
	void OnFilterTextChanged(const FText& InFilterText);

	/** Called by Slate when the filter text box is confirmed. */
	void OnFilterTextCommitted(const FText& InFilterText, ETextCommit::Type InCommitType); 

	/** Make the "Filters" menu. */
	TSharedRef<SWidget> MakeAddFilterMenu();

	/** Make the "Categories" sub-menu. */
	void MakeSelectCategoriesSubMenu(FMenuBuilder& MenuBuilder);

	ELogLevelFilter GetMessagesFilter() const;
	void OnMessagesFilterChanged(ELogLevelFilter NewFilter);

	ELogLevelFilter GetWarningsFilter() const;
	void OnWarningsFilterChanged(ELogLevelFilter NewFilter);

	ELogLevelFilter GetErrorsFilter() const;
	void OnErrorsFilterChanged(ELogLevelFilter NewFilter);
	
	/** Toggles All Categories true/false. */
	void CategoriesShowAll_Execute();

	/** Returns the state of "Show All" */
	ECheckBoxState CategoriesShowAll_IsChecked() const;

	/** Toggles the given category true/false. */
	void CategoriesSingle_Execute(FName InName);

	/** Returns the state of the given category */
	bool CategoriesSingle_IsChecked(FName InName) const;

	/** Forces re-population of the messages list */
	void Refresh();

	bool IsWordWrapEnabled() const;

	void SetWordWrapEnabled(ECheckBoxState InValue);

	void SetTimestampMode(ELogTimes::Type InValue);

	bool IsSelectedTimestampMode(ELogTimes::Type NewType);

	void AddTimestampMenuSection(FMenuBuilder& Menu);

	ELogTimes::Type GetSelectedTimestampMode();

#if WITH_EDITOR
	bool IsClearOnPIEEnabled() const;

	void SetClearOnPIE(ECheckBoxState InValue);
#endif

	static void RegisterSettingsMenu_WordWrap(FToolMenuSection& InSection);
	static void RegisterSettingsMenu_TimestampMode(FToolMenuSection& InSection);
	static void RegisterSettingsMenu_ClearOnPIE(FToolMenuSection& InSection);
	static void RegisterSettingsMenu_BrowseLogs(FToolMenuSection& InSection);
	static void RegisterSettingsMenu_OpenLogExternal(FToolMenuSection& InSection);
	FName GetSettingsMenuProfileForFlags(EOutputLogSettingsMenuFlags InFlags);
	TSharedRef<SWidget> GetSettingsMenuContent(FName InMenuProfileName);

	TSharedRef<SWidget> CreateDrawerDockButton();

	void OpenLogFileInExplorer();

	void OpenLogFileInExternalEditor();

	FReply OnDockInLayoutClicked();

protected:
	TSharedPtr<SConsoleInputBox> ConsoleInputBox;

	/** Visible messages filter */
	FOutputLogFilter Filter;

	FDelegateHandle SettingsWatchHandle;

	int32 LoggingLineLimit = 10000;

	/** True if the user has scrolled the window upwards */
	bool bIsUserScrolled = false;

	bool bShouldCreateDrawerDockButton = false;

	bool bShouldShowLoggingLimitMenu = false;

	bool bEnableLoggingLimit = false;
};

/** Output log text marshaller to convert an array of FOutputLogMessages into styled lines to be consumed by an FTextLayout */
class FOutputLogTextLayoutMarshaller : public FBaseTextLayoutMarshaller
{
public:

	static TSharedRef< FOutputLogTextLayoutMarshaller > Create(TArray< TSharedPtr<FOutputLogMessage> > InMessages, FOutputLogFilter* InFilter);

	virtual ~FOutputLogTextLayoutMarshaller();
	
	// ITextLayoutMarshaller
	virtual void SetText(const FString& SourceString, FTextLayout& TargetTextLayout) override;
	virtual void GetText(FString& TargetString, const FTextLayout& SourceTextLayout) override;

	bool AppendPendingMessage(const TCHAR* InText, const ELogVerbosity::Type InVerbosity, const FName& InCategory);
	bool SubmitPendingMessages();
	void ClearMessages();

	void CountMessages();

	int32 GetNumMessages() const;
	int32 GetNumFilteredMessages();
	int32 GetNumCachedMessages();

	void MarkMessagesCacheAsDirty();

	FName GetCategoryForLocation(const FTextLocation Location) const;

	FTextLocation GetTextLocationAt(const FVector2D& Relative) const;

	FName GetCategoryToHighlight() const { return CategoryToHighlight; }

	void SetCategoryToHighlight(FName InCategory) { CategoryToHighlight = InCategory; }

protected:

	FOutputLogTextLayoutMarshaller(TArray< TSharedPtr<FOutputLogMessage> > InMessages, FOutputLogFilter* InFilter);

	void AppendPendingMessagesToTextLayout();

	TMap<FName, float> CategoryHueMap;

	float GetCategoryHue(FName CategoryName);

	/** All log messages to show in the text box */
	TArray< TSharedPtr<FOutputLogMessage> > Messages;

	/** Messages pending add, kept separate to avoid a race condition when reading Messages */
	TArray< TSharedPtr<FOutputLogMessage> > PendingMessages;

	/** Index of the next entry in the Messages array that is pending submission to the text layout */
	int32 NextPendingMessageIndex;

	/** Holds cached numbers of messages to avoid unnecessary re-filtering */
	int32 CachedNumMessages;

	/** Flag indicating the messages count cache needs rebuilding */
	bool bNumMessagesCacheDirty;

	/** Visible messages filter */
	FOutputLogFilter* Filter;

	FName CategoryToHighlight;

	FTextLayout* TextLayout;

	/** Output log runs own its own "OutputDeviceRedirector" thread, lock against messages to prevent race conditions */
	FCriticalSection PendingMessagesCriticalSection;
};
