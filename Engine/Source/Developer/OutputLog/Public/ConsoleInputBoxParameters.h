// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class SMultiLineEditableTextBox;
class SWidget;


namespace UE::OutputLog
{
	/**
	 * Delegate that allows intercepting console commands and optionally handling them.
	 * 
	 * @param ExecutorName Name of the active command executor
	 * @param ExecCommand  Command string to evaluate
	 * @return True if the command was intercepted (handled) and should not be executed further;
	 *         false to allow normal execution.
	 */
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FInterceptConsoleCommandDelegate, const FName& /*ExecutorName*/, const FString& /*ExecCommand*/);

	/**
	 * Delegate that allows overriding the command history for a specific executor
	 *
	 * @param ExecutorName Name of the active command executor
	 * @param OutCmdHistory Command history for the executor
	 * @return True if the history was overridden; false to use the executor's default history.
	 */
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FHistoryOverrideDelegate, const FName& /*ExecutorName*/, TArray<FString>& /*OutCmdHistory*/);


	/**
	 * Parameters used to create a console input box widget
	 */
	struct FConsoleInputBoxCreationParams
	{
		/** Called when the console requests to be closed. */
		FSimpleDelegate OnCloseConsole;

		/** Called after a console command has been executed. */
		FSimpleDelegate OnConsoleCommandExecuted;

		/** Called to intercept console commands before execution. */
		FInterceptConsoleCommandDelegate OnInterceptConsoleCommand;

		/** Called to provide custom command history for a specific executor. */
		FHistoryOverrideDelegate OnHistoryOverride;
	};

	/**
	 * Result of creating a console input box widget.
	 */
	struct FConsoleInputBoxCreationResult
	{
		/** Root widget representing the console input box. */
		TSharedPtr<SWidget> ConsoleInputBox;

		/**
		 * Exposed editable text box used for command input.
		 * Can be used to control focus, text, or bind additional behavior.
		 */
		TSharedPtr<SMultiLineEditableTextBox> ExposedEditableTextBox;
	};
}
