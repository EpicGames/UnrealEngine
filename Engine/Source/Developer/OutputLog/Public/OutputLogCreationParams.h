// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

enum class EOutputLogSettingsMenuFlags
{
	None = 0x00,

	/** The clear on Pie button should not be created */
	SkipClearOnPie = 0x01,

	/** The Enable world wrapping button should not be created */
	SkipEnableWordWrapping = 0x02,

	/** Skip the button that opens the source folder of the output log module */
	SkipOpenSourceButton = 0x04,

	/** Skip the button which opens the log in a text editor */
	SkipOpenInExternalEditorButton = 0x08	
};
ENUM_CLASS_FLAGS(EOutputLogSettingsMenuFlags)

DECLARE_DELEGATE_RetVal_OneParam(bool, FAllowLogCategoryCallback, const FName);
using FDefaultCategorySelectionMap = TMap<FName, bool>;

struct FOutputLogCreationParams
{
	/** Whether to create the button for docking the log */
	bool bCreateDockInLayoutButton = false;

	/** Determines what entries the Settings drop-down will ignore */
	EOutputLogSettingsMenuFlags SettingsMenuCreationFlags = EOutputLogSettingsMenuFlags::None;

	/** Called when building the initial set of selected log categories */
	FAllowLogCategoryCallback AllowAsInitialLogCategory;

	/** Maps each log category to whether it should be selected or deselected by default. The caller is responsible to enter valid category names. */
	FDefaultCategorySelectionMap DefaultCategorySelection;

	/** Called when the console tab is closed. */
	FSimpleDelegate OnCloseConsole;

	/** Do we want to support limited console output with a predefined limit. */
	TOptional<int32> LoggingLimit;
};
