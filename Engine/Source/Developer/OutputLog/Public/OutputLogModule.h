// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConsoleInputBoxParameters.h"
#include "Misc/NamePermissionList.h"
#include "Modules/ModuleInterface.h"
#include "OutputLogCreationParams.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"

class SMultiLineEditableTextBox;
class FOutputLogHistory;
class SOutputLog;
class SDockTab;
class FSpawnTabArgs;
struct FOutputLogCreationParams;
struct FOutputLogFilter;

/** Style of the debug console */
namespace EDebugConsoleStyle
{
	enum Type
	{
		/** Shows the debug console input line with tab completion only */
		Compact,

		/** Shows a scrollable log window with the input line on the bottom */
		WithLog,
	};
};

struct FDebugConsoleDelegates
{
	FSimpleDelegate OnFocusLost;
	FSimpleDelegate OnConsoleCommandExecuted;
	FSimpleDelegate OnCloseConsole;
};

class FOutputLogModule : public IModuleInterface
{
public:
	virtual void StartupModule();
	virtual void ShutdownModule();

	static const FName ModuleName;
	static OUTPUTLOG_API FOutputLogModule& Get();
	static OUTPUTLOG_API FOutputLogModule* TryGet();

	/** Returns whether debug console widgets should be hidden */
	virtual bool ShouldHideConsole() const;

	/** Generates a console input box widget.  Remember, this widget will become invalid if the
		output log DLL is unloaded on the fly. */
	virtual TSharedRef<SWidget> MakeConsoleInputBox(TSharedPtr<SMultiLineEditableTextBox>& OutExposedEditableTextBox, const FSimpleDelegate& OnCloseConsole, const FSimpleDelegate& OnConsoleCommandExecuted) const;

	/**
	 * Creates a console input box widget
	 *
	 * @note The widget becomes invalid if the OutputLog module is unloaded at runtime
	 *
	 * @param InCreationParams Input box creation parameters
	 * @return Result structure that contains the requested widget and additional data
	 */
	virtual UE::OutputLog::FConsoleInputBoxCreationResult MakeConsoleInputBox(const UE::OutputLog::FConsoleInputBoxCreationParams& InCreationParams) const;

	virtual TSharedRef<SWidget> MakeOutputLogDrawerWidget(const FSimpleDelegate& OnCloseConsole);

	virtual TSharedRef<SWidget> MakeOutputLogWidget(const FOutputLogCreationParams& Params);

	/** Opens a debug console in the specified window, if not already open */
	virtual void ToggleDebugConsoleForWindow(const TSharedRef<SWindow>& Window, const EDebugConsoleStyle::Type InStyle, const FDebugConsoleDelegates& DebugConsoleDelegates);

	/** Closes the debug console for the specified window */
	virtual void CloseDebugConsole();

	virtual void ClearOnPIE(const bool bIsSimulating);

	virtual void FocusOutputLogConsoleBox(const TSharedRef<SWidget> OutputLogToFocus);

	virtual const TSharedPtr<SWidget> GetOutputLog() const;

	/*
	 * Suspend the collection of global log data that is used for constructing output log widgets.  This global log will perpetually collect logging information perpetually and may exhaust memory.
	 */
	virtual void SuspendGlobalLog();

	/*
	 * Resume global log collection. Note this has a memory impact as log entries are stored for the life of the running edit.
	 */
	virtual void ResumeGlobalLog();

	/** Opens and focuses on the Output Log Drawer if the status bar exists, otherwise opens and focuses on the Output Log Tab. */
	void OUTPUTLOG_API FocusOutputLog();

	/** Same as FocusOutputLog(), but also scrolls the output log window to the end. */
	virtual void FocusOutputLogAndScrollToEnd();

	const TSharedPtr<SDockTab> GetOutputLogTab() const { return OutputLogTab.Pin(); }

	struct FOutputFilterParams
	{
		TOptional<bool> bShowErrors;
		TOptional<bool> bShowWarnings;
		TOptional<bool> bShowLogs;
		TOptional<TSet<ELogVerbosity::Type>> IgnoreFilterVerbosities;
	};

	/** Change the output log's filter. If CategoriesToShow is empty, all categories will be shown. */
	void OUTPUTLOG_API UpdateOutputLogFilter(const TArray<FName>& CategoriesToShow, TOptional<bool> bShowErrors = TOptional<bool>(), TOptional<bool> bShowWarnings = TOptional<bool>(), TOptional<bool> bShowLogs = TOptional<bool>());
	void OUTPUTLOG_API UpdateOutputLogFilter(const TArray<FName>& CategoriesToShow, const FOutputFilterParams& InParams);

	/** Opens the output log tab, or brings it to front if it's already open */
	void OUTPUTLOG_API OpenOutputLog() const;

	/** Returns the value of bCycleToOutputLogDrawer from the module OutputLogSettings. This function helps StatusBar to access properties in OutputLogSetting 
	through OutputLogModule to avoid dependencies. */
	virtual bool ShouldCycleToOutputLogDrawer() const;

	TMulticastDelegate<void()> OnOutputLogDrawerCleared;

	/** Returns list used to filter which command executors are allowed */
	virtual const TUniquePtr<FNamePermissionList>& GetConsoleCommandExecutorPermissionList() const;

private:
	TSharedRef<SDockTab> SpawnOutputLogTab(const FSpawnTabArgs& Args);
	void SaveDockedTabSettings();

	TSharedRef<SDockTab> SpawnDeviceOutputLogTab(const FSpawnTabArgs& Args);

private:
	/** Our global output log app spawner */
	TSharedPtr<FOutputLogHistory> OutputLogHistory;

	/** Caches the user selected Filters as the OutputLog tab can be closed and remade multiple times */
	TUniquePtr<FOutputLogFilter> OutputLogFilterCache;

	/** Our global active output log that belongs to a tab */
	TWeakPtr<SOutputLog> OutputLog;

	/** Global tab that the output log resides in */
	TWeakPtr<SDockTab> OutputLogTab;

	/** The output log that lives in a status bar drawer */
	TWeakPtr<SOutputLog> OutputLogDrawer;

	/** Weak pointer to a debug console that's currently open, if any */
	TWeakPtr<SWidget> DebugConsole;

	/** Weak pointer to the widget to focus once they console window closes */
	TWeakPtr<SWidget> PreviousKeyboardFocusedWidget;

	/** List used to filter which command executors are allowed */
	TUniquePtr<FNamePermissionList> ConsoleCommandExecutorPermissionList;
};
