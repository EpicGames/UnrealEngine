// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Styling/SlateBrush.h"
#include "Containers/EnumAsByte.h"
#include "Misc/OutputDevice.h"
#include "Delegates/DelegateCombinations.h"
#include "OutputLogSettings.generated.h"

UENUM()
enum class ELogCategoryColorizationMode : uint8
{
	/** Do not colorize based on log categories */
	None,

	/** Colorize the entire log line, but not warnings or errors */
	ColorizeWholeLine,

	/** Colorize only the category name (including on warnings and errors) */
	ColorizeCategoryOnly,

	/** Colorize the background of the category name (including on warnings and errors) */
	ColorizeCategoryAsBadge
};

UENUM()
enum class ELogLevelFilter : uint8
{
	/** Show none of the logs at this level. */
	None,
	/** Show only the enabled logs at this level. */
	Enabled,
	/** Show all logs at this level. */
	All,
};

USTRUCT()
struct FOutputLogCategorySettings
{
	GENERATED_BODY()
	
	UPROPERTY()
	FName Name;

	UPROPERTY()
	bool bEnabled = true;
};

USTRUCT()
struct FOutputLogFilterSettings
{
	GENERATED_BODY()

	UPROPERTY()
	ELogLevelFilter MessagesFilter = ELogLevelFilter::Enabled;

	UPROPERTY()
	ELogLevelFilter WarningsFilter = ELogLevelFilter::Enabled;

	UPROPERTY()
	ELogLevelFilter ErrorsFilter = ELogLevelFilter::Enabled;

	UPROPERTY()
	FText FilterText;

	/**
	 * Note that an empty list in settings implicitly means "all" categories.
	 * This will invert state if a user explicitly disables all categories,
	 * but that is acceptable. An empty log simply looks broken.
	 */
	UPROPERTY()
	TArray<FOutputLogCategorySettings> Categories;

	UPROPERTY()
	bool bSelectNewCategories = true;
};

/**
 * Implements the Editor style settings.
 */
UCLASS(MinimalAPI, config=EditorPerProjectUserSettings)
class UOutputLogSettings : public UObject
{
	GENERATED_BODY()

public:
	UOutputLogSettings()
	{
		LogFontSize = 9;
		bCycleToOutputLogDrawer = true;
		LogTimestampMode = ELogTimes::None;
	}

	/** The font size used in the output log */
	UPROPERTY(EditAnywhere, config, Category="Output Log", meta=(DisplayName="Log Font Size", ConfigRestartRequired=true))
	int32 LogFontSize;

	/** The display mode for timestamps in the output log window */
	UPROPERTY(EditAnywhere, config, Category="Output Log", meta=(DisplayName = "Output Log Window Timestamp Mode"))
	TEnumAsByte<ELogTimes::Type> LogTimestampMode;

	/** How should categories be colorized in the output log? */
	UPROPERTY(EditAnywhere, config, Category = "Output Log")
	ELogCategoryColorizationMode CategoryColorizationMode;

	/**
	 * If checked pressing the console command shortcut will cycle between focusing the status bar console, opening the output log drawer, and back to the previous focus target. 
	 * If unchecked, the console command shortcut will only focus the status bar console
	 */
	UPROPERTY(EditAnywhere, config, Category = "Output Log", meta = (DisplayName = "Open Output Log Drawer with Console Command Shortcut"))
	bool bCycleToOutputLogDrawer;

	UPROPERTY(EditAnywhere, config, Category = "Output Log")
	bool bEnableOutputLogWordWrap;

#if WITH_EDITORONLY_DATA
	UPROPERTY(config)
	bool bEnableOutputLogClearOnPIE;
#endif

	/** The most recently used filter settings. */
	UPROPERTY(config)
	FOutputLogFilterSettings OutputLogTabFilter;

public:

	/**
	 * Returns an event delegate that is executed when a setting has changed.
	 *
	 * @return The delegate.
	 */
	DECLARE_EVENT_OneParam(UOutputLogSettings, FSettingChangedEvent, FName /*PropertyName*/);
	FSettingChangedEvent& OnSettingChanged() { return SettingChangedEvent; }

protected:
#if WITH_EDITOR
	// UObject overrides
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override
	{
		Super::PostEditChangeProperty(PropertyChangedEvent);

		SaveConfig();
		const FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
		SettingChangedEvent.Broadcast(PropertyName);
	}
#endif

private:

	// Holds an event delegate that is executed when a setting has changed.
	FSettingChangedEvent SettingChangedEvent;
};
